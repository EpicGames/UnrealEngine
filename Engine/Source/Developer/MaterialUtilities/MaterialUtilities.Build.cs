// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MaterialUtilities : ModuleRules
{
	public MaterialUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string [] {
				"Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"RenderCore",
                "Renderer",
                "RHI",
                "Landscape",
                "UnrealEd",
                "MaterialBaking",
            }
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "MeshMergeUtilities",
            }
        );

        PublicDependencyModuleNames.AddRange(
			new string [] {
                 "MeshDescription",
				 "StaticMeshDescription",
				 "GeometryCore"
			}
		);      

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "MeshMergeUtilities",
            }
        );

        CircularlyReferencedDependentModules.AddRange(
            new string[] {
                "Landscape"
            }
        );
	}
}
