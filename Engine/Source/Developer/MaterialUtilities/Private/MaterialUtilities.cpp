// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialUtilities.h"
#include "EngineDefines.h"
#include "ShowFlags.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Engine/Texture2D.h"
#include "Misc/App.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "LegacyScreenPercentageDriver.h"

#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "RendererInterface.h"
#include "EngineModule.h"
#include "ImageUtils.h"
#include "CanvasTypes.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "MaterialCompiler.h"
#include "MaterialDomain.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Materials/MaterialParameterCollection.h"
#include "LandscapeProxy.h"
#include "LandscapeComponent.h"
#include "Engine/StaticMesh.h"
#include "MeshUtilities.h"
#include "MeshRendering.h"
#include "MeshMergeData.h"
#include "PrimitiveSceneProxy.h"
#include "Templates/UniquePtr.h"
#include "TextureResource.h"


#include "IMaterialBakingModule.h"
#include "MaterialBakingStructures.h"
#include "MaterialOptions.h"

#include "StaticMeshAttributes.h"
#include "TextureCompiler.h"

#include "TriangleTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#if WITH_EDITOR
#include "DeviceProfiles/DeviceProfile.h"
#include "Tests/AutomationEditorCommon.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialUtilities)


IMPLEMENT_MODULE(FMaterialUtilities, MaterialUtilities);

DEFINE_LOG_CATEGORY_STATIC(LogMaterialUtilities, Log, All);

static TAutoConsoleVariable<int32> CVarMaterialUtilitiesWarmupFrames(
	TEXT("MaterialUtilities.WarmupFrames"),
	10,
	TEXT("Number of frames to render before each capture in order to warmup various rendering systems (VT/Nanite/etc)."));


bool FMaterialUtilities::CurrentlyRendering = false;
TArray<UTextureRenderTarget2D*> FMaterialUtilities::RenderTargetPool;

void FMaterialUtilities::StartupModule()
{
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FMaterialUtilities::OnPreGarbageCollect);
}

void FMaterialUtilities::ShutdownModule()
{
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	ClearRenderTargetPool();
}

void FMaterialUtilities::OnPreGarbageCollect()
{
	ClearRenderTargetPool();
}

UMaterialInterface* FMaterialUtilities::CreateProxyMaterialAndTextures(UPackage* OuterPackage, const FString& AssetName, const FBakeOutput& BakeOutput, const FMeshData& MeshData, const FMaterialData& MaterialData, UMaterialOptions* Options)
{
	check(MaterialData.Material);

	TArray<EMaterialProperty> SRGBEnabledProperties{ MP_BaseColor, MP_EmissiveColor, MP_SubsurfaceColor };

	// Certain material properties use differen compression settings
	TMap<EMaterialProperty, TextureCompressionSettings> SpecialCompressionSettingProperties;
	SpecialCompressionSettingProperties.Add(MP_Normal, TC_Normalmap);
	SpecialCompressionSettingProperties.Add(MP_Opacity, TC_Grayscale);
	SpecialCompressionSettingProperties.Add(MP_OpacityMask, TC_Grayscale);
	SpecialCompressionSettingProperties.Add(MP_AmbientOcclusion, TC_Grayscale);

	UMaterial* BaseMaterial = GEngine->DefaultFlattenMaterial;
	check(BaseMaterial);

	/** Create Proxy material and populate flags */
	UMaterialInstanceConstant* Material = FMaterialUtilities::CreateInstancedMaterial(BaseMaterial, OuterPackage, AssetName, RF_Public | RF_Standalone);
	check(Material);
	
	Material->BasePropertyOverrides.TwoSided = MaterialData.Material->IsTwoSided();
	Material->BasePropertyOverrides.bOverride_TwoSided = MaterialData.Material->IsTwoSided();
	Material->BasePropertyOverrides.bOverride_bIsThinSurface = MaterialData.Material->IsThinSurface();
	Material->BasePropertyOverrides.DitheredLODTransition = MaterialData.Material->IsDitheredLODTransition();
	Material->BasePropertyOverrides.bOverride_DitheredLODTransition = MaterialData.Material->IsDitheredLODTransition();

	if (!IsOpaqueBlendMode(*MaterialData.Material))
	{
		Material->BasePropertyOverrides.bOverride_BlendMode = true;
		Material->BasePropertyOverrides.BlendMode = MaterialData.Material->GetBlendMode();
	}

	FStaticParameterSet NewStaticParameterSet;
	// iterate over each property and its size
	for (auto Iterator : BakeOutput.PropertySizes)
	{
		EMaterialProperty Property = Iterator.Key;
		FIntPoint DataSize = Iterator.Value;
		// Retrieve pixel data for the current property
		const TArray<FColor>& ColorData = BakeOutput.PropertyData.FindChecked(Property);

		// Look up the property name string 
		const UEnum* PropertyEnum = StaticEnum<EMaterialProperty>();
		FName PropertyName = PropertyEnum->GetNameByValue(Property);
		FString TrimmedPropertyName = PropertyName.ToString();
		TrimmedPropertyName.RemoveFromStart(TEXT("MP_"));

		// If the pixel data isn't constant create a texture for it 
		if (ColorData.Num() > 1)
		{
			FMaterialParameterInfo ParameterInfo(*(TrimmedPropertyName + TEXT("Texture")));

			FCreateTexture2DParameters CreateParams;
			CreateParams.TextureGroup = TEXTUREGROUP_HierarchicalLOD;
			CreateParams.CompressionSettings = SpecialCompressionSettingProperties.Contains(Property) ? SpecialCompressionSettingProperties.FindChecked(Property) : TC_Default;
			CreateParams.bSRGB = SRGBEnabledProperties.Contains(Property);

			// Make sure the texture is a VT if required by the material sampler
			UTexture* DefaultTexture = nullptr;
			Material->GetTextureParameterValue(ParameterInfo, DefaultTexture);
			if (DefaultTexture)
			{
				CreateParams.bVirtualTexture = DefaultTexture->VirtualTextureStreaming;
			}

			UTexture* Texture = FMaterialUtilities::CreateTexture(OuterPackage, TEXT("T_") + AssetName + TEXT("_") + TrimmedPropertyName, DataSize, ColorData, CreateParams, RF_Public | RF_Standalone);

			// Set texture parameter value on instance material
			Material->SetTextureParameterValueEditorOnly(ParameterInfo, Texture);

			FStaticSwitchParameter SwitchParameter;
			SwitchParameter.ParameterInfo.Name = *(TEXT("Use") + TrimmedPropertyName);
			SwitchParameter.Value = true;
			SwitchParameter.bOverride = true;
			NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
		}
		else
		{
			// Otherwise set either float4 or float constant values on instance material
			FMaterialParameterInfo ParameterInfo(*(TrimmedPropertyName + TEXT("Const")));

			if (Property == MP_BaseColor || Property == MP_EmissiveColor)
			{
				if(BakeOutput.PropertyIsLinearColor.Contains(Property) && BakeOutput.PropertyIsLinearColor.FindChecked(Property))
				{
					Material->SetVectorParameterValueEditorOnly(ParameterInfo, FLinearColor(float(ColorData[0].R) / 255.0f, float(ColorData[0].G) / 255.0f, float(ColorData[0].B) / 255.0f, float(ColorData[0].A) / 255.0f));
				}
				else
				{
					Material->SetVectorParameterValueEditorOnly(ParameterInfo, FLinearColor::FromSRGBColor(ColorData[0]));
				}
			}
			else
			{
				if (BakeOutput.PropertyIsLinearColor.Contains(Property) && BakeOutput.PropertyIsLinearColor.FindChecked(Property))
				{
					Material->SetScalarParameterValueEditorOnly(ParameterInfo, float(ColorData[0].R) / 255.0f);
				}
				else
				{
					Material->SetScalarParameterValueEditorOnly(ParameterInfo, FLinearColor::FromSRGBColor(ColorData[0]).R);
				}
			}
		}
	}

	// Apply emissive scaling
	if (BakeOutput.PropertyData.Contains(MP_EmissiveColor))
	{
		if (BakeOutput.EmissiveScale != 1.0f)
		{
			FMaterialParameterInfo ParameterInfo(TEXT("EmissiveScale"));
			Material->SetScalarParameterValueEditorOnly(ParameterInfo, BakeOutput.EmissiveScale);
		}
	}

	// If the used texture coordinate was not the default UV0 set the appropriate one on the instance material
	if (MeshData.TextureCoordinateIndex != 0)
	{
		FStaticSwitchParameter SwitchParameter;
		SwitchParameter.ParameterInfo.Name = TEXT("UseCustomUV");
		SwitchParameter.Value = true;
		SwitchParameter.bOverride = true;
		NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);

		SwitchParameter.ParameterInfo.Name = *(TEXT("UseUV") + FString::FromInt(MeshData.TextureCoordinateIndex));
		NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
	}

	Material->UpdateStaticPermutation(NewStaticParameterSet);
	Material->InitStaticPermutation();

	Material->PostEditChange();

	return Material;
}

UMaterialInterface* FMaterialUtilities::CreateProxyMaterialAndTextures(const FString& PackageName, const FString& AssetName, const FBakeOutput& BakeOutput, const FMeshData& MeshData, const FMaterialData& MaterialData, UMaterialOptions* Options)
{
	UPackage* MaterialPackage = CreatePackage( *PackageName);
	check(MaterialPackage);
	MaterialPackage->FullyLoad();
	MaterialPackage->Modify();

	return CreateProxyMaterialAndTextures(MaterialPackage, AssetName, BakeOutput, MeshData, MaterialData, Options);	
}

/*------------------------------------------------------------------------------
	Helper classes for render material to texture
------------------------------------------------------------------------------*/
struct FExportMaterialCompiler : public FProxyMaterialCompiler
{
	FExportMaterialCompiler(FMaterialCompiler* InCompiler) :
		FProxyMaterialCompiler(InCompiler)
	{}

	// gets value stored by SetMaterialProperty()
	virtual EShaderFrequency GetCurrentShaderFrequency() const override
	{
		// not used by Lightmass
		return SF_Pixel;
	}

	virtual FMaterialShadingModelField GetMaterialShadingModels() const override
	{
		// not used by Lightmass
		return MSM_MAX;
	}

	virtual FMaterialShadingModelField GetCompiledShadingModels() const override
	{
		// not used by Lightmass
		return MSM_MAX;
	}

	virtual int32 WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets) override
	{
#if WITH_EDITOR
		return Compiler->MaterialBakingWorldPosition();
#else
		return Compiler->WorldPosition(WorldPositionIncludedOffsets);
#endif
	}

	virtual int32 ObjectWorldPosition(EPositionOrigin OriginType) override
	{
		return Compiler->ObjectWorldPosition(OriginType);
	}

	virtual int32 DistanceCullFade() override
	{
		return Compiler->Constant(1.0f);
	}

	virtual int32 ActorWorldPosition(EPositionOrigin OriginType) override
	{
		return Compiler->ActorWorldPosition(OriginType);
	}

	virtual int32 ParticleRelativeTime() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleMotionBlurFade() override
	{
		return Compiler->Constant(1.0f);
	}
	
	virtual int32 PixelNormalWS() override
	{
		// Current returning vertex normal since pixel normal will contain incorrect data (normal calculated from uv data used as vertex positions to render out the material)
		return Compiler->VertexNormal();
	}
	
	virtual int32 ParticleRandom() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleDirection() override
	{
		return Compiler->Constant3(0.0f, 0.0f, 0.0f);
	}

	virtual int32 ParticleSpeed() override
	{
		return Compiler->Constant(0.0f);
	}
	
	virtual int32 ParticleSize() override
	{
		return Compiler->Constant2(0.0f,0.0f);
	}

	virtual int32 ParticleSpriteRotation() override
	{
		return Compiler->Constant2(0.0f, 0.0f);
	}

	virtual int32 ObjectRadius() override
	{
		return Compiler->Constant(500);
	}

	virtual int32 ObjectBounds() override
	{
		return Compiler->ObjectBounds();
	}

	virtual int32 PreSkinnedLocalBounds(int32 OutputIndex) override
	{
		return Compiler->PreSkinnedLocalBounds(OutputIndex);
	}

	virtual int32 CameraVector() override
	{
		return Compiler->Constant3(0.0f, 0.0f, 1.0f);
	}
	
	virtual int32 ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal) override
	{
		return Compiler->ReflectionAboutCustomWorldNormal(CustomWorldNormal, bNormalizeCustomWorldNormal);
	}

	virtual int32 VertexColor() override
	{
		return Compiler->VertexColor(); 
	}

	virtual int32 PreSkinnedPosition() override
	{
		return Compiler->PreSkinnedPosition();
	}

	virtual int32 PreSkinnedNormal() override
	{
		return Compiler->PreSkinnedNormal();
	}

	virtual int32 VertexInterpolator(uint32 InterpolatorIndex) override
	{
		return Compiler->VertexInterpolator(InterpolatorIndex);
	}

	virtual int32 LightVector() override
	{
		return Compiler->LightVector();
	}

	virtual int32 ReflectionVector() override
	{
		return Compiler->ReflectionVector();
	}

	virtual int32 AtmosphericFogColor(int32 WorldPosition, EPositionOrigin PositionOrigin) override
	{
		return INDEX_NONE;
	}

	virtual int32 PrecomputedAOMask() override 
	{ 
		return Compiler->PrecomputedAOMask();
	}

#if WITH_EDITOR
	virtual int32 MaterialBakingWorldPosition() override
	{
		return Compiler->MaterialBakingWorldPosition();
	}
#endif

	virtual int32 AccessCollectionParameter(UMaterialParameterCollection* ParameterCollection, int32 ParameterIndex, int32 ComponentIndex) override
	{
		if (!ParameterCollection || ParameterIndex == -1)
		{
			return INDEX_NONE;
		}

		// Collect names of all parameters
		TArray<FName> ParameterNames;
		ParameterCollection->GetParameterNames(ParameterNames, /*bVectorParameters=*/ false);
		int32 NumScalarParameters = ParameterNames.Num();
		ParameterCollection->GetParameterNames(ParameterNames, /*bVectorParameters=*/ true);

		// Find a parameter corresponding to ParameterIndex/ComponentIndex pair
		int32 Index;
		for (Index = 0; Index < ParameterNames.Num(); Index++)
		{
			FGuid ParameterId = ParameterCollection->GetParameterId(ParameterNames[Index]);
			int32 CheckParameterIndex, CheckComponentIndex;
			ParameterCollection->GetParameterIndex(ParameterId, CheckParameterIndex, CheckComponentIndex);
			if (CheckParameterIndex == ParameterIndex && CheckComponentIndex == ComponentIndex)
			{
				// Found
				break;
			}
		}
		if (Index >= ParameterNames.Num())
		{
			// Not found, should not happen
			return INDEX_NONE;
		}

		// Create code for parameter
		if (Index < NumScalarParameters)
		{
			const FCollectionScalarParameter* ScalarParameter = ParameterCollection->GetScalarParameterByName(ParameterNames[Index]);
			check(ScalarParameter);
			return Constant(ScalarParameter->DefaultValue);
		}
		else
		{
			const FCollectionVectorParameter* VectorParameter = ParameterCollection->GetVectorParameterByName(ParameterNames[Index]);
			check(VectorParameter);
			const FLinearColor& Color = VectorParameter->DefaultValue;
			return Constant4(Color.R, Color.G, Color.B, Color.A);
		}
	}
	
	virtual EMaterialCompilerType GetCompilerType() const override { return EMaterialCompilerType::MaterialProxy; }
};

class FExportMaterialProxy : public FEditorMaterialResourceBase, public FMaterialRenderProxy
{
public:
	FExportMaterialProxy()
		: FEditorMaterialResourceBase()
		, FMaterialRenderProxy(TEXT("FExportMaterialProxy"))
	{
		SetQualityLevelProperties(GMaxRHIShaderPlatform);
	}

	FExportMaterialProxy(UMaterialInterface* InMaterialInterface, EMaterialProperty InPropertyToCompile)
		: FEditorMaterialResourceBase()
		, FMaterialRenderProxy(GetPathNameSafe(InMaterialInterface->GetMaterial()))
		, MaterialInterface(InMaterialInterface)
		, PropertyToCompile(InPropertyToCompile)
	{
		SetQualityLevelProperties(GMaxRHIShaderPlatform);
		Material = InMaterialInterface->GetMaterial();
		ReferencedTextures = InMaterialInterface->GetDefaultTextures();
		ReferencedTextureCollections = MaterialInterface->GetReferencedTextureCollections();

		FPlatformMisc::CreateGuid(Id);

		FMaterialResource* Resource = InMaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform);

		FMaterialShaderMapId ResourceId;
		Resource->BuildShaderMapId(ResourceId, nullptr);

		{
			TArray<FShaderType*> ShaderTypes;
			TArray<FVertexFactoryType*> VFTypes;
			TArray<const FShaderPipelineType*> ShaderPipelineTypes;
			GetDependentShaderAndVFTypes(ResourceId.LayoutParams, ShaderTypes, ShaderPipelineTypes, VFTypes);

			// Overwrite the shader map Id's dependencies with ones that came from the FMaterial actually being compiled (this)
			// This is necessary as we change FMaterial attributes like GetShadingModels(), which factor into the ShouldCache functions that determine dependent shader types
			ResourceId.SetShaderDependencies(ShaderTypes, ShaderPipelineTypes, VFTypes, GMaxRHIShaderPlatform);
		}

		// Override with a special usage so we won't re-use the shader map used by the material for rendering
		switch (InPropertyToCompile)
		{
		case MP_BaseColor: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportBaseColor; break;
		case MP_Specular: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportSpecular; break;
		case MP_Normal: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportNormal; break;
		case MP_Tangent: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportTangent; break;
		case MP_Metallic: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportMetallic; break;
		case MP_Roughness: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportRoughness; break;
		case MP_Anisotropy: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportAnisotropy; break;
		case MP_AmbientOcclusion: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportAO; break;
		case MP_EmissiveColor: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportEmissive; break;
		case MP_Opacity: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportOpacity; break;
		case MP_OpacityMask: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportOpacity; break;
		case MP_SubsurfaceColor: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportSubSurfaceColor; break;
		default:
			ensureMsgf(false, TEXT("ExportMaterial has no usage for property %i.  Will likely reuse the normal rendering shader and crash later with a parameter mismatch"), (int32)InPropertyToCompile);
			break;
		};
		
		CacheShaders(ResourceId);
	}

	virtual bool IsUsedWithStaticLighting() const { return true; }

	/** This override is required otherwise the shaders aren't ready for use when the surface is rendered resulting in a blank image */
	virtual bool RequiresSynchronousCompilation() const override { return true; };

	/**
	* Should the shader for this material with the given platform, shader type and vertex 
	* factory type combination be compiled
	*
	* @param ShaderType	Which shader is being compiled
	* @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	*
	* @return true if the shader should be compiled
	*/
	virtual bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const override
	{
		// Always cache - decreases performance but avoids missing shaders during exports.
		return true;
	}

	////////////////
	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		if (GetRenderingThreadShaderMap())
		{
			return this;
		}
		return nullptr;
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override
	{
		return MaterialInterface->GetRenderProxy()->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}

	// Material properties.
	/** Entry point for compiling a specific material property.  This must call SetMaterialProperty. */
	virtual int32 CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const override
	{
		// needs to be called in this function!!
		Compiler->SetMaterialProperty(Property, OverrideShaderFrequency, bUsePreviousFrameTime);

		int32 Ret = CompilePropertyAndSetMaterialPropertyWithoutCast(Property, Compiler);

		return Compiler->ForceCast(Ret, FMaterialAttributeDefinitionMap::GetValueType(Property), MFCF_ExactMatch | MFCF_ReplicateValue);
	}

	/** helper for CompilePropertyAndSetMaterialProperty() */
	int32 CompilePropertyAndSetMaterialPropertyWithoutCast(EMaterialProperty Property, FMaterialCompiler* Compiler) const
	{
		if (Property == MP_EmissiveColor || Property == MP_SubsurfaceColor)
		{
			const bool bIsOpaqueOrMasked = IsOpaqueOrMaskedBlendMode(*MaterialInterface);
			UMaterial* ProxyMaterial = MaterialInterface->GetMaterial();
			check(ProxyMaterial);
			FExportMaterialCompiler ProxyCompiler(Compiler);
			const uint32 ForceCast_Exact_Replicate = MFCF_ForceCast | MFCF_ExactMatch | MFCF_ReplicateValue;
									
			switch (PropertyToCompile)
			{
			case MP_EmissiveColor:
				// Emissive is ALWAYS returned...
				return MaterialInterface->CompileProperty(&ProxyCompiler, MP_EmissiveColor, ForceCast_Exact_Replicate);
			case MP_BaseColor:
				// Only return for Opaque and Masked...
				if (bIsOpaqueOrMasked)
				{
				return MaterialInterface->CompileProperty(&ProxyCompiler, MP_BaseColor, ForceCast_Exact_Replicate);
				}
				break;
			case MP_Specular: 
			case MP_Roughness:
			case MP_Anisotropy:
			case MP_Metallic:
			case MP_AmbientOcclusion:
			case MP_SubsurfaceColor:
				// Only return for Opaque and Masked...
				if (bIsOpaqueOrMasked)
				{
					return MaterialInterface->CompileProperty(&ProxyCompiler, PropertyToCompile, ForceCast_Exact_Replicate);
				}
				break;
			case MP_Normal:
			case MP_Tangent:
				// Only return for Opaque and Masked...
				if (bIsOpaqueOrMasked)
				{
					return Compiler->Add( 
							Compiler->Mul(MaterialInterface->CompileProperty(&ProxyCompiler, PropertyToCompile, ForceCast_Exact_Replicate), Compiler->Constant(0.5f)), // [-1,1] * 0.5
							Compiler->Constant(0.5f)); // [-0.5,0.5] + 0.5
				}
				break;
			case MP_ShadingModel:
				return MaterialInterface->CompileProperty(&ProxyCompiler, MP_ShadingModel);
			case MP_SurfaceThickness:
				return MaterialInterface->CompileProperty(&ProxyCompiler, MP_SurfaceThickness);
			case MP_FrontMaterial:
				return MaterialInterface->CompileProperty(&ProxyCompiler, MP_FrontMaterial);
			default:
				return Compiler->Constant(1.0f);
			}
	
			return Compiler->Constant(0.0f);
		}
		else if (Property == MP_WorldPositionOffset || Property == MP_Displacement)
		{
			//This property MUST return 0 as a default or during the process of rendering textures out for lightmass to use, pixels will be off by 1.
			return Compiler->Constant3(0.0f, 0.0f, 0.0f);
		}
		else if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
		{
			// Pass through customized UVs
			return MaterialInterface->CompileProperty(Compiler, Property);
		}
		else if (Property == MP_ShadingModel)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_ShadingModel);
		}
		else if (Property == MP_SurfaceThickness)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_SurfaceThickness);
		}
		else if (Property == MP_FrontMaterial)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_FrontMaterial);
		}
		else
		{
			// Return the proper default value for each property so the translator doesn't
			// falsely detect usage of features like WPO, anisotropy, etc.
			return FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property);
		}
	}

	virtual FString GetMaterialUsageDescription() const override
	{
		return FString::Printf(TEXT("ExportMaterialRenderer %s"), MaterialInterface ? *MaterialInterface->GetName() : TEXT("NULL"));
	}
	virtual EMaterialDomain GetMaterialDomain() const override
	{
		if (Material)
		{
			return Material->MaterialDomain;
		}
		return MD_Surface;
	}
	virtual bool IsTwoSided() const  override
	{ 
		if (MaterialInterface)
		{
			return MaterialInterface->IsTwoSided();
		}
		return false;
	}
	virtual bool IsThinSurface() const  override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsThinSurface();
		}
		return false;
	}
	virtual bool IsDitheredLODTransition() const  override
	{ 
		if (MaterialInterface)
		{
			return MaterialInterface->IsDitheredLODTransition();
		}
		return false;
	}
	virtual bool IsLightFunction() const override
	{
		if (Material)
		{
			return (Material->MaterialDomain == MD_LightFunction);
		}
		return false;
	}
	virtual bool IsDeferredDecal() const override
	{
		return Material && Material->MaterialDomain == MD_DeferredDecal;
	}
	virtual bool IsUIMaterial() const override
	{
		return Material && Material->MaterialDomain == MD_UI;
	}
	virtual bool IsSpecialEngineMaterial() const override
	{
		if (Material)
		{
			return (Material->bUsedAsSpecialEngineMaterial == 1);
		}
		return true;
	}
	virtual bool IsWireframe() const override
	{
		if (Material)
		{
			return (Material->Wireframe == 1);
		}
		return false;
	}
	virtual bool IsMasked() const override									{ return false; }
	virtual enum EBlendMode GetBlendMode() const override					{ return BLEND_Opaque; }
	virtual enum ERefractionMode GetRefractionMode() const override			{ return Material ? (ERefractionMode)Material->RefractionMethod : RM_None; }
	virtual bool GetRootNodeOverridesDefaultRefraction()const override		{ return false; }
	virtual FMaterialShadingModelField GetShadingModels() const override	{ return MSM_Unlit; }
	virtual bool IsShadingModelFromMaterialExpression() const override		{ return false; }
	virtual float GetOpacityMaskClipValue() const override					{ return 0.5f; }
	virtual bool GetCastDynamicShadowAsMasked() const override				{ return false; }
	virtual FString GetFriendlyName() const override { return FString::Printf(TEXT("FExportMaterialRenderer %s"), MaterialInterface ? *MaterialInterface->GetName() : TEXT("NULL")); }
	/**
	* Should shaders compiled for this material be saved to disk?
	*/
	virtual bool IsPersistent() const override { return false; }
	virtual FGuid GetMaterialId() const override { return Id; }

	virtual UMaterialInterface* GetMaterialInterface() const override
	{
		return MaterialInterface;
	}

	friend FArchive& operator<< ( FArchive& Ar, FExportMaterialProxy& V )
	{
		return Ar << V.MaterialInterface;
	}

	static bool WillFillData(bool bIsOpaque, EMaterialProperty InMaterialProperty)
	{
		if (InMaterialProperty == MP_EmissiveColor)
		{
			return true;
		}

		if (bIsOpaque)
		{
			switch (InMaterialProperty)
			{
			case MP_BaseColor:			return true;
			case MP_Specular:			return true;
			case MP_Normal:				return true;
			case MP_Tangent:			return true;
			case MP_Metallic:			return true;
			case MP_Roughness:			return true;
			case MP_Anisotropy:			return true;
			case MP_AmbientOcclusion:	return true;
			}
		}
		return false;
	}

	virtual bool IsVolumetricPrimitive() const override
	{
		return Material && Material->MaterialDomain == MD_Volume;
	}

	virtual void GatherExpressionsForCustomInterpolators(TArray<UMaterialExpression*>& OutExpressions) const override
	{
		if(Material)
		{
			Material->GetAllExpressionsForCustomInterpolators(OutExpressions);
		}
	}

	virtual bool CheckInValidStateForCompilation(class FMaterialCompiler* Compiler) const override
	{
		return Material && Material->CheckInValidStateForCompilation(Compiler);
	}

private:
	/** The material interface for this proxy */
	UMaterialInterface* MaterialInterface;
	UMaterial* Material;	
	/** The property to compile for rendering the sample */
	EMaterialProperty PropertyToCompile;
	FGuid Id;
};

/**
 * Render the scene to the provided canvas. Will potentially perform the render multiple times, depending on the value of
 * the CVarMaterialUtilitiesWarmupFrames CVar. This is needed to ensure various rendering systems are primed properly before capturing
 * the scene.
 */
static void PerformSceneRender(FCanvas& Canvas, FSceneViewFamily& ViewFamily, FSceneView& View, bool bWithWarmup)
{
	if (bWithWarmup)
	{
		for (int32 i = 0; i < CVarMaterialUtilitiesWarmupFrames.GetValueOnGameThread(); i++)
		{
			// Increment frame index when warming up virtual textures.
			// This is necessary to jitter the virtual texture feedback location.
			View.OverrideFrameIndexValue = i;

			GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);
		}
	}

	// Do final capture with deterministic frame index 0.
	View.OverrideFrameIndexValue = 0;

	GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);
}

static void RenderSceneToTexture(
	FSceneInterface* Scene,
	const FName& VisualizationMode, 
	const FVector& ViewOrigin,
	const FMatrix& ViewRotationMatrix, 
	const FMatrix& ProjectionMatrix,
	const TSet<FPrimitiveComponentId>& ShowOnlyPrimitives,
	const TSet<FPrimitiveComponentId>& HiddenPrimitives, 
	FIntPoint TargetSize,
	float TargetGamma,
	bool bPerformWarmpup,
	TArray<FColor>& OutSamples)
{
	auto RenderTargetTexture = NewObject<UTextureRenderTarget2D>();
	check(RenderTargetTexture);
	RenderTargetTexture->AddToRoot();
	RenderTargetTexture->ClearColor = FLinearColor::Transparent;
	RenderTargetTexture->TargetGamma = TargetGamma;
	RenderTargetTexture->InitCustomFormat(TargetSize.X, TargetSize.Y, PF_FloatRGBA, false);
	FTextureRenderTargetResource* RenderTargetResource = RenderTargetTexture->GameThread_GetRenderTargetResource();

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(RenderTargetResource, Scene, FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime::GetTimeSinceAppStart())
		);

	// To enable visualization mode
	ViewFamily.EngineShowFlags.SetPostProcessing(true);
	ViewFamily.EngineShowFlags.SetVisualizeBuffer(true);
	ViewFamily.EngineShowFlags.SetTonemapper(false);
	ViewFamily.EngineShowFlags.SetScreenPercentage(false);
	ViewFamily.EngineShowFlags.SetMegaLights(false);

	FSceneViewStateReference ViewState;
	ViewState.Allocate(Scene->GetFeatureLevel());

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.SetViewRectangle(FIntRect(0, 0, TargetSize.X, TargetSize.Y));
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.ViewOrigin = ViewOrigin;
	ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
	ViewInitOptions.ProjectionMatrix = ProjectionMatrix;
	ViewInitOptions.SceneViewStateInterface = ViewState.GetReference();

	// If no "show only" primitives are provided, we must pass an unset TOptional - otherwise an empty set will mean no primitive should be visible.
	ViewInitOptions.ShowOnlyPrimitives = !ShowOnlyPrimitives.IsEmpty() ? TOptional<TSet<FPrimitiveComponentId>>(ShowOnlyPrimitives) : TOptional<TSet<FPrimitiveComponentId>>();
	ViewInitOptions.HiddenPrimitives = HiddenPrimitives;
		
	FSceneView* NewView = new FSceneView(ViewInitOptions);
	NewView->CurrentBufferVisualizationMode = VisualizationMode;
	ViewFamily.Views.Add(NewView);

	ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
		ViewFamily, /* GlobalResolutionFraction = */ 1.0f));

	FCanvas Canvas(RenderTargetResource, NULL, FGameTime::GetTimeSinceAppStart(), Scene->GetFeatureLevel());
	Canvas.Clear(FLinearColor::Transparent);

	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(Scene));
	for (const FSceneViewExtensionRef& Extension : ViewFamily.ViewExtensions)
	{
		Extension->SetupViewFamily(ViewFamily);
		Extension->SetupView(ViewFamily, *NewView);
	}

	PerformSceneRender(Canvas, ViewFamily, *NewView, bPerformWarmpup);

	// Copy the contents of the remote texture to system memory
	OutSamples.SetNumUninitialized(TargetSize.X*TargetSize.Y);
	FReadSurfaceDataFlags ReadSurfaceDataFlags;
	ReadSurfaceDataFlags.SetLinearToGamma(false);
	RenderTargetResource->ReadPixelsPtr(OutSamples.GetData(), ReadSurfaceDataFlags, FIntRect(0, 0, TargetSize.X, TargetSize.Y));
					
	RenderTargetTexture->RemoveFromRoot();
	RenderTargetTexture = nullptr;
}


static void RenderSceneToTextures(
	FSceneInterface* Scene,
	const FVector& ViewOrigin,
	const FMatrix& ViewRotationMatrix,
	const FMatrix& ProjectionMatrix,
	const TSet<FPrimitiveComponentId>& ShowOnlyPrimitives,
	const TSet<FPrimitiveComponentId>& HiddenPrimitives,
	FFlattenMaterial& OutFlattenMaterial)
{
	TMap<EFlattenMaterialProperties, TPair<FName, float>>	SupportedProperties; // Property -> VisualisationMode|Gamma
	SupportedProperties.Add(EFlattenMaterialProperties::Diffuse) = TPair<FName, float>(FName("BaseColor"), true);	// BaseColor to gamma space
	SupportedProperties.Add(EFlattenMaterialProperties::Normal) = TPair<FName, float>(FName("WorldNormal"), false);	// Dump normal texture in linear space
	SupportedProperties.Add(EFlattenMaterialProperties::Metallic) = TPair<FName, float>(FName("Metallic"), false);	// Dump metallic texture in linear space
	SupportedProperties.Add(EFlattenMaterialProperties::Roughness) = TPair<FName, float>(FName("Roughness"), true);	// Roughness material powers color by 2.2, transform it back to linear
	SupportedProperties.Add(EFlattenMaterialProperties::Specular) = TPair<FName, float>(FName("Specular"), false);	// Dump specular texture in linear space

	bool bPerformWarmpup = true;
	for (int32 PropertyIndex = 0; PropertyIndex < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++PropertyIndex)
	{
		const EFlattenMaterialProperties Property = (EFlattenMaterialProperties)PropertyIndex;
		if (OutFlattenMaterial.ShouldGenerateDataForProperty(Property))
		{
			TPair<FName, float>* PropertyInfo = SupportedProperties.Find(Property);
			if (PropertyInfo)
			{
				TArray<FColor>& Samples = OutFlattenMaterial.GetPropertySamples(Property);
				const FIntPoint& Size = OutFlattenMaterial.GetPropertySize(Property);
				const FName VisModeName = PropertyInfo->Key;
				const float Gamma = PropertyInfo->Value ? 2.2f : 1.0f;

				RenderSceneToTexture(Scene, VisModeName, ViewOrigin, ViewRotationMatrix, ProjectionMatrix, ShowOnlyPrimitives, HiddenPrimitives, Size, Gamma, bPerformWarmpup, Samples);
				bPerformWarmpup = false; // Perform warmup only on first render
			}
			else
			{
				UE_LOGF(LogMaterialUtilities, Error, "RenderSceneToTextures - Ignoring unsupported property");
			}
		}
	}
}

FIntPoint FMaterialUtilities::FindMaxTextureSize(UMaterialInterface* InMaterialInterface, FIntPoint MinimumSize)
{
	// static lod settings so that we only initialize them once
	UTextureLODSettings* GameTextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();

	TArray<UTexture*> MaterialTextures;

	InMaterialInterface->GetUsedTextures(MaterialTextures, GetCurrentMaterialQualityLevelChecked(), GetMaxShaderPlatformChecked());
	FTextureCompilingManager::Get().FinishCompilation(MaterialTextures);

	// find the largest texture in the list (applying it's LOD bias)
	FIntPoint MaxSize = MinimumSize;
	for (int32 TexIndex = 0; TexIndex < MaterialTextures.Num(); TexIndex++)
	{
		UTexture* Texture = MaterialTextures[TexIndex];

		if (Texture == NULL)
		{
			continue;
		}

		// get the max size of the texture
		FIntPoint LocalSize(0, 0);
		if (Texture->IsA(UTexture2D::StaticClass()))
		{
			UTexture2D* Tex2D = (UTexture2D*)Texture;
			LocalSize = FIntPoint(Tex2D->GetSizeX(), Tex2D->GetSizeY());
		}
		else if (Texture->IsA(UTextureCube::StaticClass()))
		{
			UTextureCube* TexCube = (UTextureCube*)Texture;
			LocalSize = FIntPoint(TexCube->GetSizeX(), TexCube->GetSizeY());
		}
		else if (Texture->IsA(UTexture2DArray::StaticClass()))
		{
			UTexture2DArray* TexArray = (UTexture2DArray*)Texture;
			LocalSize = FIntPoint(TexArray->GetSizeX(), TexArray->GetSizeY());
		}

		int32 LocalBias = GameTextureLODSettings->CalculateLODBias(Texture);

		// bias the texture size based on LOD group
		FIntPoint BiasedLocalSize(LocalSize.X >> LocalBias, LocalSize.Y >> LocalBias);

		MaxSize.X = FMath::Max(BiasedLocalSize.X, MaxSize.X);
		MaxSize.Y = FMath::Max(BiasedLocalSize.Y, MaxSize.Y);
	}

	return MaxSize;
}

bool FMaterialUtilities::SupportsExport(bool bIsOpaque, EMaterialProperty InMaterialProperty)
{
	return FExportMaterialProxy::WillFillData(bIsOpaque, InMaterialProperty);
}

static bool ExportLandscapeMaterial(const ALandscapeProxy* InLandscape, const TSet<FPrimitiveComponentId>& ShowOnlyPrimitives, const TSet<FPrimitiveComponentId>& HiddenPrimitives, FFlattenMaterial& OutFlattenMaterial)
{
	check(InLandscape);

	FIntRect LandscapeRect = InLandscape->GetBoundingRect();
	FVector MidPoint = FVector(LandscapeRect.Min, 0.f) + FVector(LandscapeRect.Size(), 0.f)*0.5f;
		
	FVector LandscapeCenter = InLandscape->GetTransform().TransformPosition(MidPoint);
	FVector LandscapeExtent = FVector(LandscapeRect.Size(), 0.f)*InLandscape->GetActorScale()*0.5f; 

	FVector ViewOrigin = LandscapeCenter;
	FMatrix ViewRotationMatrix = FInverseRotationMatrix(InLandscape->GetActorRotation());
	ViewRotationMatrix*= FMatrix(FPlane(1,	0,	0,	0),
							FPlane(0,	-1,	0,	0),
							FPlane(0,	0,	-1,	0),
							FPlane(0,	0,	0,	1));
				
	const FMatrix::FReal ZOffset = UE_OLD_WORLD_MAX;
	FMatrix ProjectionMatrix =  FReversedZOrthoMatrix(
		LandscapeExtent.X,
		LandscapeExtent.Y,
		0.5f / ZOffset,
		ZOffset);

	FSceneInterface* Scene = InLandscape->GetWorld()->Scene;

	RenderSceneToTextures(Scene, ViewOrigin, ViewRotationMatrix, ProjectionMatrix, ShowOnlyPrimitives, HiddenPrimitives, OutFlattenMaterial);

	OutFlattenMaterial.MaterialId = InLandscape->GetLandscapeGuid();
	return true;
}

bool FMaterialUtilities::ExportLandscapeMaterial(const ALandscapeProxy* InLandscape, FFlattenMaterial& OutFlattenMaterial)
{
	bool bExportSuccess = false;

	if (InLandscape)
	{
		TSet<FPrimitiveComponentId> ShowOnlyPrimitives;

		// Include all landscape components scene proxies
		for (ULandscapeComponent* LandscapeComponent : InLandscape->LandscapeComponents)
		{
			if (LandscapeComponent && LandscapeComponent->SceneProxy)
			{
				ShowOnlyPrimitives.Add(LandscapeComponent->SceneProxy->GetPrimitiveComponentId());
			}
		}

		// Include Nanite landscape scene proxy - these are the ones that are actually visible when rendering LS with Nanite support
		if (InLandscape->HasNaniteComponents())
		{
			ShowOnlyPrimitives.Append(InLandscape->GetNanitePrimitiveComponentIds());
		}

		bExportSuccess = ::ExportLandscapeMaterial(InLandscape, ShowOnlyPrimitives, {}, OutFlattenMaterial);
	}
	
	if (!bExportSuccess)
	{
		UE_LOGF(LogMaterialUtilities, Warning, "ExportLandscapeMaterial: Failed to export material for the provided ALandcapeProxy (%ls)", InLandscape ? *InLandscape->GetName() : TEXT("<null>"));
	}

	return bExportSuccess;
}

bool FMaterialUtilities::ExportLandscapeMaterial(const ALandscapeProxy* InLandscape, const TSet<FPrimitiveComponentId>& HiddenPrimitives, FFlattenMaterial& OutFlattenMaterial)
{
	return ::ExportLandscapeMaterial(InLandscape, {}, HiddenPrimitives, OutFlattenMaterial);
}

UMaterial* FMaterialUtilities::CreateMaterial(const FFlattenMaterial& InFlattenMaterial, UPackage* InOuter, const FString& BaseName, EObjectFlags Flags, const struct FMaterialProxySettings& MaterialProxySettings, TArray<UObject*>& OutGeneratedAssets, const TextureGroup& InTextureGroup /*= TEXTUREGROUP_World*/)
{
	// Base name for a new assets
	// In case outer is null BaseName has to be long package name
	if (InOuter == nullptr && FPackageName::IsShortPackageName(BaseName))
	{
		UE_LOGF(LogMaterialUtilities, Warning, "Invalid long package name: '%ls'.", *BaseName);
		return nullptr;
	}

	const FString AssetBaseName = FPackageName::GetShortName(BaseName);
	const FString AssetBasePath = InOuter ? TEXT("") : FPackageName::GetLongPackagePath(BaseName);
				
	// Create material
	const FString MaterialAssetName = TEXT("M_") + AssetBaseName;
	UPackage* MaterialOuter = InOuter;
	if (MaterialOuter == NULL)
	{
		MaterialOuter = CreatePackage( *(AssetBasePath / MaterialAssetName));
		MaterialOuter->FullyLoad();
		MaterialOuter->Modify();
	}

	UMaterial* Material = NewObject<UMaterial>(MaterialOuter, FName(*MaterialAssetName), Flags);
	Material->TwoSided = false;
	Material->DitheredLODTransition = InFlattenMaterial.bDitheredLODTransition;
	Material->SetShadingModel(MSM_DefaultLit);
	OutGeneratedAssets.Add(Material);

	UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

	int32 MaterialNodeY = -150;
	int32 MaterialNodeStepY = 180;

	// BaseColor
	const TArray<FColor>& DiffuseSamples = InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse);
	if (DiffuseSamples.Num() > 1)
	{
		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_D");
		const FString AssetLongName = AssetBasePath / AssetName;
		const bool bSRGB = true;
		UTexture2D* Texture = CreateTexture(InOuter, AssetLongName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse), DiffuseSamples, TC_Default, InTextureGroup, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);
			
		auto BasecolorExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		BasecolorExpression->Texture = Texture;
		BasecolorExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
		BasecolorExpression->MaterialExpressionEditorX = -400;
		BasecolorExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(BasecolorExpression);
		MaterialEditorOnly->BaseColor.Expression = BasecolorExpression;

		MaterialNodeY += MaterialNodeStepY;
	}
	else if (DiffuseSamples.Num() == 1)
	{
		// Set Roughness to constant
		FLinearColor BaseColor = FLinearColor(DiffuseSamples[0]);
		auto BaseColorExpression = NewObject<UMaterialExpressionConstant4Vector>(Material);
		BaseColorExpression->Constant = BaseColor;
		BaseColorExpression->MaterialExpressionEditorX = -400;
		BaseColorExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(BaseColorExpression);
		MaterialEditorOnly->BaseColor.Expression = BaseColorExpression;

		MaterialNodeY += MaterialNodeStepY;
	}

	// Whether or not a material property is baked down
	const bool bHasMetallic = InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Metallic) && !InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Metallic);
	const bool bHasSpecular = InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Specular) && !InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Specular);
	const bool bHasRoughness = InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Roughness) && !InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Roughness);

	// Number of material properties baked down to textures
	const int BakedMaterialPropertyCount = bHasMetallic + bHasRoughness + bHasSpecular;

	// Check for same texture sizes
	bool bSameTextureSize = true;	

	int32 SampleCount = 0;
	FIntPoint MergedSize(0,0);
	for (int32 PropertyIndex = 0; PropertyIndex < 3; ++PropertyIndex)
	{
		EFlattenMaterialProperties Property = (EFlattenMaterialProperties)(PropertyIndex + (int32)EFlattenMaterialProperties::Metallic);
		const bool HasProperty = InFlattenMaterial.DoesPropertyContainData(Property) && !InFlattenMaterial.IsPropertyConstant(Property);
		FIntPoint PropertySize = InFlattenMaterial.GetPropertySize(Property);
		SampleCount = (bHasMetallic && SampleCount == 0) ? InFlattenMaterial.GetPropertySamples(Property).Num() : SampleCount;
		MergedSize = (bHasMetallic && MergedSize.X == 0) ? PropertySize : MergedSize;
	}
	bSameTextureSize &= bHasMetallic ? (SampleCount == InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Num()) : true;
	bSameTextureSize &= bHasSpecular ? (SampleCount == InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Num()) : true;
	bSameTextureSize &= bHasRoughness ? (SampleCount == InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Num()) : true;

	// Merge values into one texture if more than one material property exists
	if (BakedMaterialPropertyCount > 1 && bSameTextureSize)
	{
		// Metallic = R, Specular = G, Roughness = B
		TArray<FColor> MergedSamples;
		MergedSamples.AddZeroed(SampleCount);
		
		// R G B masks
		const uint32 ColorMask[3] = { 0x00FF0000, 0x0000FF00, 0x000000FF };

		for (int32 PropertyIndex = 0; PropertyIndex < 3; ++PropertyIndex)
		{
			EFlattenMaterialProperties Property = (EFlattenMaterialProperties)(PropertyIndex + (int32)EFlattenMaterialProperties::Metallic);
			const bool HasProperty = InFlattenMaterial.DoesPropertyContainData(Property) && !InFlattenMaterial.IsPropertyConstant(Property);

			if (HasProperty)
			{
					const TArray<FColor>& PropertySamples = InFlattenMaterial.GetPropertySamples(Property);
					// OR masked values (samples initialized to zero, so no random data)
					for (int32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex)
					{
						MergedSamples[SampleIndex].DWColor() |= (PropertySamples[SampleIndex].DWColor() & ColorMask[PropertyIndex]);
					}
			}
		}

		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_MSR");
		const bool bSRGB = true;
		UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, MergedSize, MergedSamples, TC_Default, InTextureGroup, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);

		auto MergedExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		MergedExpression->Texture = Texture;
		MergedExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
		MergedExpression->MaterialExpressionEditorX = -400;
		MergedExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(MergedExpression);

		// Metallic
		if (bHasMetallic)
		{
			MaterialEditorOnly->Metallic.Expression = MergedExpression;
			MaterialEditorOnly->Metallic.Mask = MaterialEditorOnly->Metallic.Expression->GetOutputs()[0].Mask;
			MaterialEditorOnly->Metallic.MaskR = 1;
			MaterialEditorOnly->Metallic.MaskG = 0;
			MaterialEditorOnly->Metallic.MaskB = 0;
			MaterialEditorOnly->Metallic.MaskA = 0;
		}

		// Specular
		if (bHasSpecular)
		{
			MaterialEditorOnly->Specular.Expression = MergedExpression;
			MaterialEditorOnly->Specular.Mask = MaterialEditorOnly->Specular.Expression->GetOutputs()[0].Mask;
			MaterialEditorOnly->Specular.MaskR = 0;
			MaterialEditorOnly->Specular.MaskG = 1;
			MaterialEditorOnly->Specular.MaskB = 0;
			MaterialEditorOnly->Specular.MaskA = 0;
		}

		// Roughness
		if (bHasRoughness)
		{
			MaterialEditorOnly->Roughness.Expression = MergedExpression;
			MaterialEditorOnly->Roughness.Mask = MaterialEditorOnly->Roughness.Expression->GetOutputs()[0].Mask;
			MaterialEditorOnly->Roughness.MaskR = 0;
			MaterialEditorOnly->Roughness.MaskG = 0;
			MaterialEditorOnly->Roughness.MaskB = 1;
			MaterialEditorOnly->Roughness.MaskA = 0;
		}

		MaterialNodeY += MaterialNodeStepY;
	}
	else
	{
		// Metallic
		if (bHasMetallic && MaterialProxySettings.bMetallicMap)
		{
			const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_M");
			const bool bSRGB = true;
			UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic), TC_Default, InTextureGroup, Flags, bSRGB);
			OutGeneratedAssets.Add(Texture);

			auto MetallicExpression = NewObject<UMaterialExpressionTextureSample>(Material);
			MetallicExpression->Texture = Texture;
			MetallicExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
			MetallicExpression->MaterialExpressionEditorX = -400;
			MetallicExpression->MaterialExpressionEditorY = MaterialNodeY;
			Material->GetExpressionCollection().AddExpression(MetallicExpression);
			MaterialEditorOnly->Metallic.Expression = MetallicExpression;

			MaterialNodeY += MaterialNodeStepY;
		}

		// Specular
		if (bHasSpecular && MaterialProxySettings.bSpecularMap)
		{
			const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_S");
			const bool bSRGB = true;
			UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular), TC_Default, InTextureGroup, Flags, bSRGB);
			OutGeneratedAssets.Add(Texture);

			auto SpecularExpression = NewObject<UMaterialExpressionTextureSample>(Material);
			SpecularExpression->Texture = Texture;
			SpecularExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
			SpecularExpression->MaterialExpressionEditorX = -400;
			SpecularExpression->MaterialExpressionEditorY = MaterialNodeY;
			Material->GetExpressionCollection().AddExpression(SpecularExpression);
			MaterialEditorOnly->Specular.Expression = SpecularExpression;

			MaterialNodeY += MaterialNodeStepY;
		}

		// Roughness
		if (bHasRoughness && MaterialProxySettings.bRoughnessMap)
		{
			const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_R");
			const bool bSRGB = true;
			UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness), TC_Default, InTextureGroup, Flags, bSRGB);
			OutGeneratedAssets.Add(Texture);

			auto RoughnessExpression = NewObject<UMaterialExpressionTextureSample>(Material);
			RoughnessExpression->Texture = Texture;
			RoughnessExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
			RoughnessExpression->MaterialExpressionEditorX = -400;
			RoughnessExpression->MaterialExpressionEditorY = MaterialNodeY;
			Material->GetExpressionCollection().AddExpression(RoughnessExpression);
			MaterialEditorOnly->Roughness.Expression = RoughnessExpression;

			MaterialNodeY += MaterialNodeStepY;
		}
	}

	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Metallic) || !MaterialProxySettings.bMetallicMap)
	{
		auto MetallicExpression = NewObject<UMaterialExpressionConstant>(Material);
		MetallicExpression->R = MaterialProxySettings.bMetallicMap ? FLinearColor(InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic)[0]).R : MaterialProxySettings.MetallicConstant;
		MetallicExpression->MaterialExpressionEditorX = -400;
		MetallicExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(MetallicExpression);
		MaterialEditorOnly->Metallic.Expression = MetallicExpression;

		MaterialNodeY += MaterialNodeStepY;
	}

	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Specular ) || !MaterialProxySettings.bSpecularMap)
	{
		// Set Specular to constant
		auto SpecularExpression = NewObject<UMaterialExpressionConstant>(Material);
		SpecularExpression->R = MaterialProxySettings.bSpecularMap ? FLinearColor(InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular)[0]).R : MaterialProxySettings.SpecularConstant;
		SpecularExpression->MaterialExpressionEditorX = -400;
		SpecularExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(SpecularExpression);
		MaterialEditorOnly->Specular.Expression = SpecularExpression;

		MaterialNodeY += MaterialNodeStepY;
	}
	
	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Roughness) || !MaterialProxySettings.bRoughnessMap)
	{
		// Set Roughness to constant
		auto RoughnessExpression = NewObject<UMaterialExpressionConstant>(Material);
		RoughnessExpression->R = MaterialProxySettings.bRoughnessMap ? FLinearColor(InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness)[0]).R : MaterialProxySettings.RoughnessConstant;
		RoughnessExpression->MaterialExpressionEditorX = -400;
		RoughnessExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(RoughnessExpression);
		MaterialEditorOnly->Roughness.Expression = RoughnessExpression;

		MaterialNodeY += MaterialNodeStepY;
	}

	// Normal
	if (InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Normal) && !InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Normal))
	{
		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_N");
		const bool bSRGB = false;
		UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal), TC_Normalmap, (InTextureGroup != TEXTUREGROUP_World) ? InTextureGroup : TEXTUREGROUP_WorldNormalMap, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);
			
		auto NormalExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		NormalExpression->Texture = Texture;
		NormalExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Normal;
		NormalExpression->MaterialExpressionEditorX = -400;
		NormalExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(NormalExpression);
		MaterialEditorOnly->Normal.Expression = NormalExpression;

		MaterialNodeY+= MaterialNodeStepY;
	}

	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Emissive))
	{
		// Set Emissive to constant
		FColor EmissiveColor = InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive)[0];
		// Don't have to deal with black emissive color
		if (EmissiveColor != FColor(0, 0, 0, 255))
		{
			auto EmissiveColorExpression = NewObject<UMaterialExpressionConstant4Vector>(Material);
			EmissiveColorExpression->Constant = EmissiveColor.ReinterpretAsLinear() * InFlattenMaterial.EmissiveScale;
			EmissiveColorExpression->MaterialExpressionEditorX = -400;
			EmissiveColorExpression->MaterialExpressionEditorY = MaterialNodeY;
			Material->GetExpressionCollection().AddExpression(EmissiveColorExpression);
			MaterialEditorOnly->EmissiveColor.Expression = EmissiveColorExpression;

			MaterialNodeY += MaterialNodeStepY;
		}
	}
	else if (InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Emissive) && !InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Emissive))
	{
		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_E");
		const bool bSRGB = true;
		UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive), TC_Default, InTextureGroup, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);

		//Assign emissive color to the material
		UMaterialExpressionTextureSample* EmissiveColorExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		EmissiveColorExpression->Texture = Texture;
		EmissiveColorExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
		EmissiveColorExpression->MaterialExpressionEditorX = -400;
		EmissiveColorExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(EmissiveColorExpression);

		UMaterialExpressionMultiply* EmissiveColorScale = NewObject<UMaterialExpressionMultiply>(Material);
		EmissiveColorScale->A.Expression = EmissiveColorExpression;
		EmissiveColorScale->ConstB = InFlattenMaterial.EmissiveScale;
		EmissiveColorScale->MaterialExpressionEditorX = -200;
		EmissiveColorScale->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(EmissiveColorScale);

		MaterialEditorOnly->EmissiveColor.Expression = EmissiveColorScale;
		MaterialNodeY += MaterialNodeStepY;
	}

	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Opacity))
	{
		// Set Opacity to constant
		FLinearColor Opacity = FLinearColor(InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity)[0]);
		auto OpacityExpression = NewObject<UMaterialExpressionConstant>(Material);
		OpacityExpression->R = Opacity.R;
		OpacityExpression->MaterialExpressionEditorX = -400;
		OpacityExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(OpacityExpression);
		MaterialEditorOnly->Opacity.Expression = OpacityExpression;

		MaterialNodeY += MaterialNodeStepY;
	}
	else if (InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Opacity))
	{
		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_O");
		const bool bSRGB = true;
		UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity), TC_Default, InTextureGroup, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);

		//Assign opacity to the material
		UMaterialExpressionTextureSample* OpacityExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		OpacityExpression->Texture = Texture;
		OpacityExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
		OpacityExpression->MaterialExpressionEditorX = -400;
		OpacityExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(OpacityExpression);
		MaterialEditorOnly->Opacity.Expression = OpacityExpression;
		MaterialNodeY += MaterialNodeStepY;
	}

	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::OpacityMask))
	{
		// Set OpacityMask to constant
		FLinearColor OpacityMask = FLinearColor(InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask)[0]);
		auto OpacityMaskExpression = NewObject<UMaterialExpressionConstant>(Material);
		OpacityMaskExpression->R = OpacityMask.R;
		OpacityMaskExpression->MaterialExpressionEditorX = -400;
		OpacityMaskExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(OpacityMaskExpression);
		MaterialEditorOnly->OpacityMask.Expression = OpacityMaskExpression;

		MaterialNodeY += MaterialNodeStepY;
	}
	else if (InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::OpacityMask))
	{
		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_OM");
		const bool bSRGB = true;
		UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask), TC_Default, InTextureGroup, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);

		//Assign opacity to the material
		UMaterialExpressionTextureSample* OpacityMaskExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		OpacityMaskExpression->Texture = Texture;
		OpacityMaskExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
		OpacityMaskExpression->MaterialExpressionEditorX = -400;
		OpacityMaskExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(OpacityMaskExpression);
		MaterialEditorOnly->OpacityMask.Expression = OpacityMaskExpression;
		MaterialNodeY += MaterialNodeStepY;
	}

	if (InFlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::SubSurface))
	{
		// Set Emissive to constant
		FColor SubSurfaceColor = InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::SubSurface)[0];

		// Don't have to deal with black sub surface color
		if (SubSurfaceColor != FColor(0, 0, 0, 255))
		{
			auto SubSurfaceColorExpression = NewObject<UMaterialExpressionConstant4Vector>(Material);
			SubSurfaceColorExpression->Constant = (SubSurfaceColor.ReinterpretAsLinear());
			SubSurfaceColorExpression->MaterialExpressionEditorX = -400;
			SubSurfaceColorExpression->MaterialExpressionEditorY = MaterialNodeY;
			Material->GetExpressionCollection().AddExpression(SubSurfaceColorExpression);
			MaterialEditorOnly->SubsurfaceColor.Expression = SubSurfaceColorExpression;

			MaterialNodeY += MaterialNodeStepY;
		}

		Material->SetShadingModel(MSM_Subsurface);

	}
	else if (InFlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::SubSurface))
	{
		const FString AssetName = TEXT("T_") + AssetBaseName + TEXT("_SSC");
		const bool bSRGB = true;
		UTexture2D* Texture = CreateTexture(InOuter, AssetBasePath / AssetName, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::SubSurface), InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::SubSurface), TC_Default, InTextureGroup, Flags, bSRGB);
		OutGeneratedAssets.Add(Texture);

		//Assign emissive color to the material
		UMaterialExpressionTextureSample* SubSurfaceColorExpression = NewObject<UMaterialExpressionTextureSample>(Material);
		SubSurfaceColorExpression->Texture = Texture;
		SubSurfaceColorExpression->SamplerType = EMaterialSamplerType::SAMPLERTYPE_Color;
		SubSurfaceColorExpression->MaterialExpressionEditorX = -400;
		SubSurfaceColorExpression->MaterialExpressionEditorY = MaterialNodeY;
		Material->GetExpressionCollection().AddExpression(SubSurfaceColorExpression);

		MaterialEditorOnly->SubsurfaceColor.Expression = SubSurfaceColorExpression;
		MaterialNodeY += MaterialNodeStepY;

		Material->SetShadingModel(MSM_Subsurface);
	}

							
	Material->PostEditChange();
	return Material;
}

UMaterialInstanceConstant* FMaterialUtilities::CreateInstancedMaterial(UMaterialInterface* BaseMaterial, UPackage* InOuter, const FString& BaseName, EObjectFlags Flags)
{
	// Base name for a new assets
	// In case outer is null BaseName has to be long package name
	if (InOuter == nullptr && FPackageName::IsShortPackageName(BaseName))
	{
		UE_LOGF(LogMaterialUtilities, Warning, "Invalid long package name: '%ls'.", *BaseName);
		return nullptr;
	}

	const FString AssetBaseName = FPackageName::GetShortName(BaseName);
	const FString AssetBasePath = InOuter ? TEXT("") : FPackageName::GetLongPackagePath(BaseName);

	// Create material
	const FString MaterialAssetName = TEXT("MI_") + AssetBaseName;
	UPackage* MaterialOuter = InOuter;
	if (MaterialOuter == NULL)
	{		
		MaterialOuter = CreatePackage( *(AssetBasePath / MaterialAssetName));
		MaterialOuter->FullyLoad();
		MaterialOuter->Modify();
	}

	// We need to check for this due to the change in material object type, this causes a clash of path/type with old assets that were generated, so we delete the old (resident) UMaterial objects
	UObject* ExistingPackage = FindObject<UMaterial>(MaterialOuter, *MaterialAssetName);
	if (ExistingPackage && !ExistingPackage->IsA<UMaterialInstanceConstant>())
	{
#if WITH_AUTOMATION_TESTS
		FAutomationEditorCommonUtils::NullReferencesToObject(ExistingPackage);		
#endif // WITH_AUTOMATION_TESTS
		ExistingPackage->MarkAsGarbage();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
	}

	UMaterialInstanceConstant* MaterialInstance = NewObject<UMaterialInstanceConstant>(MaterialOuter, FName(*MaterialAssetName), Flags);
	checkf(MaterialInstance, TEXT("Failed to create instanced material"));
	MaterialInstance->Parent = BaseMaterial;
	
	return MaterialInstance;
}

UTexture2D* FMaterialUtilities::CreateTexture(UPackage* Outer, const FString& AssetLongName, FIntPoint Size, const TArray<FColor>& Samples, TextureCompressionSettings CompressionSettings, TextureGroup LODGroup, EObjectFlags Flags, bool bSRGB, const FGuid& SourceGuidHash)
{
	FCreateTexture2DParameters TexParams;
	TexParams.bUseAlpha = false;
	TexParams.CompressionSettings = CompressionSettings;
	TexParams.bDeferCompression = true;
	TexParams.bSRGB = bSRGB;
	TexParams.SourceGuidHash = SourceGuidHash;
	TexParams.TextureGroup = LODGroup;

	return CreateTexture(Outer, AssetLongName, Size, Samples, TexParams, Flags);
}

UTexture2D* FMaterialUtilities::CreateTexture(UPackage* Outer, const FString& AssetLongName, FIntPoint Size, const TArray<FColor>& Samples, const FCreateTexture2DParameters& CreateParams, EObjectFlags Flags)
{
	if (Outer == nullptr)
	{
		Outer = CreatePackage(*AssetLongName);
		Outer->FullyLoad();
		Outer->Modify();
	}

	return FImageUtils::CreateTexture2D(Size.X, Size.Y, Samples, Outer, FPackageName::GetShortName(AssetLongName), Flags, CreateParams);
}

bool FMaterialUtilities::ExportBaseColor(ULandscapeComponent* LandscapeComponent, int32 TextureSize, TArray<FColor>& OutSamples)
{
	ALandscapeProxy* LandscapeProxy = LandscapeComponent->GetLandscapeProxy();

	FIntPoint ComponentOrigin = LandscapeComponent->GetSectionBase() - LandscapeProxy->GetSectionBase();
	FIntPoint ComponentSize(LandscapeComponent->ComponentSizeQuads, LandscapeComponent->ComponentSizeQuads);
	FVector MidPoint = FVector(ComponentOrigin, 0.f) + FVector(ComponentSize, 0.f)*0.5f;

	FVector LandscapeCenter = LandscapeProxy->GetTransform().TransformPosition(MidPoint);
	FVector LandscapeExtent = FVector(ComponentSize, 0.f)*LandscapeProxy->GetActorScale()*0.5f;

	FVector ViewOrigin = LandscapeCenter;
	FMatrix ViewRotationMatrix = FInverseRotationMatrix(LandscapeProxy->GetActorRotation());
	ViewRotationMatrix *= FMatrix(FPlane(1, 0, 0, 0),
		FPlane(0, -1, 0, 0),
		FPlane(0, 0, -1, 0),
		FPlane(0, 0, 0, 1));

	const FMatrix::FReal ZOffset = UE_OLD_WORLD_MAX;
	FMatrix ProjectionMatrix = FReversedZOrthoMatrix(
		LandscapeExtent.X,
		LandscapeExtent.Y,
		0.5f / ZOffset,
		ZOffset);

	FSceneInterface* Scene = LandscapeProxy->GetWorld()->Scene;

	// Hide all but the component
	TSet<FPrimitiveComponentId> ShowOnlyPrimitives = { LandscapeComponent->SceneProxy->GetPrimitiveComponentId() };
	TSet<FPrimitiveComponentId> HiddenPrimitives;
				
	FIntPoint TargetSize(TextureSize, TextureSize);

	// Render diffuse texture using BufferVisualizationMode=BaseColor
	static const FName BaseColorName("BaseColor");
	const float BaseColorGamma = 2.2f;
	RenderSceneToTexture(Scene, BaseColorName, ViewOrigin, ViewRotationMatrix, ProjectionMatrix, ShowOnlyPrimitives, HiddenPrimitives, TargetSize, BaseColorGamma, true, OutSamples);
	return true;
}

FFlattenMaterial FMaterialUtilities::CreateFlattenMaterialWithSettings(const FMaterialProxySettings& InMaterialLODSettings)
{
	// Create new material.
	FFlattenMaterial Material;

	FIntPoint MaximumSize = FIntPoint::ZeroValue;

	if (InMaterialLODSettings.TextureSizingType == TextureSizingType_UseManualOverrideTextureSize)
	{
		// If the user is manually overriding the texture size, make sure we have the max texture size to render with
		MaximumSize = (MaximumSize.X < InMaterialLODSettings.DiffuseTextureSize.X) ? InMaterialLODSettings.DiffuseTextureSize : MaximumSize ;
		MaximumSize = (InMaterialLODSettings.bSpecularMap && (MaximumSize.X < InMaterialLODSettings.SpecularTextureSize.X)) ? InMaterialLODSettings.SpecularTextureSize	:	MaximumSize;
		MaximumSize = (InMaterialLODSettings.bMetallicMap && (MaximumSize.X < InMaterialLODSettings.MetallicTextureSize.X)) ? InMaterialLODSettings.MetallicTextureSize	:	MaximumSize;
		MaximumSize = (InMaterialLODSettings.bRoughnessMap && (MaximumSize.X < InMaterialLODSettings.RoughnessTextureSize.X)) ? InMaterialLODSettings.RoughnessTextureSize :	MaximumSize;
		MaximumSize = (InMaterialLODSettings.bNormalMap && (MaximumSize.X < InMaterialLODSettings.NormalTextureSize.X)) ? InMaterialLODSettings.NormalTextureSize :			MaximumSize;
		MaximumSize = (InMaterialLODSettings.bEmissiveMap && (MaximumSize.X < InMaterialLODSettings.EmissiveTextureSize.X)) ? InMaterialLODSettings.EmissiveTextureSize :	MaximumSize;
		MaximumSize = (InMaterialLODSettings.bOpacityMap && (MaximumSize.X < InMaterialLODSettings.OpacityTextureSize.X)) ? InMaterialLODSettings.OpacityTextureSize :		MaximumSize;
		MaximumSize = (InMaterialLODSettings.bOpacityMaskMap && (MaximumSize.X < InMaterialLODSettings.OpacityMaskTextureSize.X)) ? InMaterialLODSettings.OpacityMaskTextureSize : MaximumSize;

		Material.RenderSize = MaximumSize;
		Material.SetPropertySize(EFlattenMaterialProperties::Diffuse, InMaterialLODSettings.DiffuseTextureSize);
		Material.SetPropertySize(EFlattenMaterialProperties::Specular, InMaterialLODSettings.bSpecularMap ? InMaterialLODSettings.SpecularTextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Metallic, InMaterialLODSettings.bMetallicMap ? InMaterialLODSettings.MetallicTextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Roughness, InMaterialLODSettings.bRoughnessMap ? InMaterialLODSettings.RoughnessTextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Normal, InMaterialLODSettings.bNormalMap ? InMaterialLODSettings.NormalTextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Emissive, InMaterialLODSettings.bEmissiveMap ? InMaterialLODSettings.EmissiveTextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Opacity, InMaterialLODSettings.bOpacityMap ? InMaterialLODSettings.OpacityTextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::OpacityMask, InMaterialLODSettings.bOpacityMaskMap ? InMaterialLODSettings.OpacityMaskTextureSize : FIntPoint::ZeroValue);
	}
	else if (InMaterialLODSettings.TextureSizingType == TextureSizingType_UseAutomaticBiasedSizes)
	{
		Material.RenderSize = InMaterialLODSettings.TextureSize;
		
		int NormalSize = InMaterialLODSettings.TextureSize.X;
		int DiffuseSize = FMath::Max(InMaterialLODSettings.TextureSize.X >> 1, 32);
		int OtherSize = FMath::Max(InMaterialLODSettings.TextureSize.X >> 2, 16);
		FIntPoint PropertiesSize = FIntPoint(OtherSize, OtherSize);

		Material.SetPropertySize(EFlattenMaterialProperties::Diffuse, FIntPoint(DiffuseSize, DiffuseSize));
		Material.SetPropertySize(EFlattenMaterialProperties::Normal, (InMaterialLODSettings.bNormalMap) ? FIntPoint(NormalSize, NormalSize) : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Specular, (InMaterialLODSettings.bSpecularMap) ? PropertiesSize : FIntPoint::ZeroValue );
		Material.SetPropertySize(EFlattenMaterialProperties::Metallic, (InMaterialLODSettings.bMetallicMap) ? PropertiesSize : FIntPoint::ZeroValue );
		Material.SetPropertySize(EFlattenMaterialProperties::Roughness, (InMaterialLODSettings.bRoughnessMap) ? PropertiesSize : FIntPoint::ZeroValue );
		Material.SetPropertySize(EFlattenMaterialProperties::Emissive, (InMaterialLODSettings.bEmissiveMap) ? PropertiesSize : FIntPoint::ZeroValue );
		Material.SetPropertySize(EFlattenMaterialProperties::Opacity, (InMaterialLODSettings.bOpacityMap) ? PropertiesSize : FIntPoint::ZeroValue );
		Material.SetPropertySize(EFlattenMaterialProperties::OpacityMask, (InMaterialLODSettings.bOpacityMaskMap) ? PropertiesSize : FIntPoint::ZeroValue);
	}
	else if (InMaterialLODSettings.TextureSizingType == TextureSizingType_UseSingleTextureSize)
	{
		Material.RenderSize = InMaterialLODSettings.TextureSize;
		Material.SetPropertySize(EFlattenMaterialProperties::Diffuse, InMaterialLODSettings.TextureSize);
		Material.SetPropertySize(EFlattenMaterialProperties::Specular, (InMaterialLODSettings.bSpecularMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Metallic, (InMaterialLODSettings.bMetallicMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Roughness, (InMaterialLODSettings.bRoughnessMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Normal, (InMaterialLODSettings.bNormalMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Emissive, (InMaterialLODSettings.bEmissiveMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::Opacity, (InMaterialLODSettings.bOpacityMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
		Material.SetPropertySize(EFlattenMaterialProperties::OpacityMask, (InMaterialLODSettings.bOpacityMaskMap) ? InMaterialLODSettings.TextureSize : FIntPoint::ZeroValue);
	}
	else
	{
		UE_LOGF(LogMaterialUtilities, Error, "Unsupported TextureSizingType value. You should resolve the material texture size first with ResolveTextureSize()");
	}

	return Material;
}

void FMaterialUtilities::AnalyzeMaterial(UMaterialInterface* InMaterial, const struct FMaterialProxySettings& InMaterialSettings, int32& OutNumTexCoords, bool& OutRequiresVertexData)
{
	OutRequiresVertexData = false;
	OutNumTexCoords = 0;

	bool PropertyBeingBaked[MP_Tangent + 1];
	PropertyBeingBaked[MP_BaseColor] = true;
	PropertyBeingBaked[MP_Specular] = InMaterialSettings.bSpecularMap;
	PropertyBeingBaked[MP_Roughness] = InMaterialSettings.bRoughnessMap;
	PropertyBeingBaked[MP_Anisotropy] = InMaterialSettings.bAnisotropyMap;
	PropertyBeingBaked[MP_Metallic] = InMaterialSettings.bMetallicMap;
	PropertyBeingBaked[MP_Normal] = InMaterialSettings.bNormalMap;
	PropertyBeingBaked[MP_Tangent] = InMaterialSettings.bTangentMap;
	PropertyBeingBaked[MP_Metallic] = InMaterialSettings.bOpacityMap;
	PropertyBeingBaked[MP_EmissiveColor] = InMaterialSettings.bEmissiveMap;

	for (int32 PropertyIndex = 0; PropertyIndex < UE_ARRAY_COUNT(PropertyBeingBaked); ++PropertyIndex)
	{
		if (PropertyBeingBaked[PropertyIndex])
		{
			EMaterialProperty Property = (EMaterialProperty)PropertyIndex;

			if (PropertyIndex == MP_Opacity)
			{
				if (IsMaskedBlendMode(*InMaterial))
				{
					Property = MP_OpacityMask;
				}
				else if (IsTranslucentBlendMode(*InMaterial))
				{
					Property = MP_Opacity;
				}
				else
				{
					continue;
				}
			}

			// Analyze this material channel.
			int32 NumTextureCoordinates = 0;
			bool bUseVertexData = false;
			InMaterial->AnalyzeMaterialProperty(Property, NumTextureCoordinates, bUseVertexData);
			// Accumulate data.
			OutNumTexCoords = FMath::Max(NumTextureCoordinates, OutNumTexCoords);
			OutRequiresVertexData |= bUseVertexData;
		}
	}
}

void FMaterialUtilities::AnalyzeMaterial(class UMaterialInterface* InMaterial, const TArray<EMaterialProperty>& Properties, int32& OutNumTexCoords, bool& OutRequiresVertexData)
{
	OutRequiresVertexData = false;
	OutNumTexCoords = 0;
	
	for (EMaterialProperty Property : Properties)
	{
		if (Property == MP_Opacity)
		{
			if (IsMaskedBlendMode(*InMaterial))
			{
				Property = MP_OpacityMask;
			}
			else if (IsTranslucentBlendMode(*InMaterial))
			{
				Property = MP_Opacity;
			}
			else
			{
				continue;
			}
		}

		// Analyze this material channel.
		int32 NumTextureCoordinates = 0;
		bool bUseVertexData = false;
		InMaterial->AnalyzeMaterialProperty(Property, NumTextureCoordinates, bUseVertexData);
		// Accumulate data.
		OutNumTexCoords = FMath::Max(NumTextureCoordinates, OutNumTexCoords);
		OutRequiresVertexData |= bUseVertexData;
	}
}

void FMaterialUtilities::RemapUniqueMaterialIndices(const TArray<FSectionInfo>& InSections, const TArray<FRawMeshExt>& InMeshData, const TMap<FMeshIdAndLOD, TArray<int32> >& InMaterialMap, const FMaterialProxySettings& InMaterialProxySettings, const bool bBakeVertexData, const bool bMergeMaterials, TArray<bool>& OutMeshShouldBakeVertexData, TMap<FMeshIdAndLOD, TArray<int32> >& OutMaterialMap, TArray<FSectionInfo>& OutSections)
{
	// Gather material properties
	TMap<UMaterialInterface*, int32> MaterialNumTexCoords;
	TMap<UMaterialInterface*, bool>  MaterialUseVertexData;

	for (int32 SectionIndex = 0; SectionIndex < InSections.Num(); SectionIndex++)
	{
		const FSectionInfo& Section = InSections[SectionIndex];
		if (MaterialNumTexCoords.Find(Section.Material) != nullptr)
		{
			// This material was already processed.
			continue;
		}

		if (!bBakeVertexData || !bMergeMaterials)
		{
			// We are not baking vertex data at all, don't analyze materials.
			MaterialNumTexCoords.Add(Section.Material, 0);
			MaterialUseVertexData.Add(Section.Material, false);
			continue;
		}
		int32 NumTexCoords = 0;
		bool bUseVertexData = false;
		FMaterialUtilities::AnalyzeMaterial(Section.Material, InMaterialProxySettings, NumTexCoords, bUseVertexData);
		MaterialNumTexCoords.Add(Section.Material, NumTexCoords);
		MaterialUseVertexData.Add(Section.Material, bUseVertexData);
	}

	for (int32 MeshIndex = 0; MeshIndex < InMeshData.Num(); MeshIndex++)
	{
		for (int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
		{
			TVertexAttributesRef<FVector3f> VertexPositions = InMeshData[MeshIndex].MeshLODData[LODIndex].RawMesh->GetVertexPositions();

			if (InMeshData[MeshIndex].bShouldExportLOD[LODIndex])
			{
				checkf(VertexPositions.GetNumElements(), TEXT("No vertex data found in mesh LOD"));
				
				const TArray<int32>& MeshMaterialMap = InMaterialMap[FMeshIdAndLOD(MeshIndex, LODIndex)];
				int32 NumTexCoords = 0;
				bool bUseVertexData = false;
				// Accumulate data of all materials.
				for (int32 LocalMaterialIndex = 0; LocalMaterialIndex < MeshMaterialMap.Num(); LocalMaterialIndex++)
				{
					UMaterialInterface* Material = InSections[MeshMaterialMap[LocalMaterialIndex]].Material;
					NumTexCoords = FMath::Max(NumTexCoords, MaterialNumTexCoords[Material]);
					bUseVertexData |= MaterialUseVertexData[Material];
				}

				// Store data.
				OutMeshShouldBakeVertexData[MeshIndex] |= bUseVertexData || (NumTexCoords >= 2);
			}
		}
	}

	// Build new material map.
	// Structure used to simplify material merging.
	struct FMeshMaterialData
	{
		FSectionInfo SectionInfo;
		UStaticMesh* Mesh;
		bool bHasVertexColors;

		FMeshMaterialData(const FSectionInfo& InSection, UStaticMesh* InMesh, bool bInHasVertexColors)
			: SectionInfo(InSection)
			, Mesh(InMesh)
			, bHasVertexColors(bInHasVertexColors)
		{
		}

		bool operator==(const FMeshMaterialData& Other) const
		{
			return SectionInfo == Other.SectionInfo && Mesh == Other.Mesh && bHasVertexColors == Other.bHasVertexColors;
		}
	};

	TArray<FMeshMaterialData> MeshMaterialData;
	OutMaterialMap.Empty();
	for (int32 MeshIndex = 0; MeshIndex < InMeshData.Num(); MeshIndex++)
	{
		for (int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
		{
			TVertexAttributesRef<FVector3f> VertexPositions = InMeshData[MeshIndex].MeshLODData[LODIndex].RawMesh->GetVertexPositions();

			if (InMeshData[MeshIndex].bShouldExportLOD[LODIndex])
			{
				checkf(VertexPositions.GetNumElements(), TEXT("No vertex data found in mesh LOD"));
		
				const TArray<int32>& MeshMaterialMap = InMaterialMap[FMeshIdAndLOD(MeshIndex, LODIndex)];
				TArray<int32>& NewMeshMaterialMap = OutMaterialMap.Add(FMeshIdAndLOD(MeshIndex, LODIndex));
				UStaticMesh* StaticMesh = InMeshData[MeshIndex].SourceStaticMesh;

				if (!OutMeshShouldBakeVertexData[MeshIndex])
				{
					// No vertex data needed - could merge materials with other meshes.
					// Set to 'nullptr' if don't need to bake vertex data to be able to merge materials with any meshes
					// which don't require vertex data baking too.
					StaticMesh = nullptr;

					for (int32 LocalMaterialIndex = 0; LocalMaterialIndex < MeshMaterialMap.Num(); LocalMaterialIndex++)
					{
						FMeshMaterialData Data(InSections[MeshMaterialMap[LocalMaterialIndex]], StaticMesh, false);
						int32 Index = MeshMaterialData.Find(Data);
						if (Index == INDEX_NONE)
						{
							// Not found, add new entry.
							Index = MeshMaterialData.Add(Data);
						}
						NewMeshMaterialMap.Add(Index);
					}
				}
				else
				{
					// Mesh with vertex data baking, and with vertex colors - don't share materials at all.
					for (int32 LocalMaterialIndex = 0; LocalMaterialIndex < MeshMaterialMap.Num(); LocalMaterialIndex++)
					{
						FMeshMaterialData Data(InSections[MeshMaterialMap[LocalMaterialIndex]], StaticMesh, true);
						int32 Index = MeshMaterialData.Add(Data);
						NewMeshMaterialMap.Add(Index);
					}
				}		
			}
		}
	}

	// Build new material list - simply extract MeshMaterialData[i].Material.
	OutSections.Empty();
	OutSections.AddDefaulted(MeshMaterialData.Num());
	for (int32 MaterialIndex = 0; MaterialIndex < MeshMaterialData.Num(); MaterialIndex++)
	{
		OutSections[MaterialIndex] = MeshMaterialData[MaterialIndex].SectionInfo;
	}
}

void FMaterialUtilities::OptimizeFlattenMaterial(FFlattenMaterial& InFlattenMaterial)
{
	// Try to optimize each individual property sample
	for (int32 PropertyIndex = 0; PropertyIndex < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++PropertyIndex)
	{
		EFlattenMaterialProperties Property = (EFlattenMaterialProperties)PropertyIndex;
		FIntPoint Size = InFlattenMaterial.GetPropertySize(Property);
		OptimizeSampleArray(InFlattenMaterial.GetPropertySamples(Property), Size);
		InFlattenMaterial.SetPropertySize(Property, Size);
	}
}

void FMaterialUtilities::ResizeFlattenMaterial(FFlattenMaterial& InFlattenMaterial, const struct FMeshProxySettings& InMeshProxySettings)
{
	const FMaterialProxySettings& MaterialSettings = InMeshProxySettings.MaterialSettings;
	if (MaterialSettings.TextureSizingType == TextureSizingType_UseAutomaticBiasedSizes)
	{
		int NormalSizeX, DiffuseSizeX, PropertiesSizeX;
		NormalSizeX = MaterialSettings.TextureSize.X;
		DiffuseSizeX = FMath::Max(MaterialSettings.TextureSize.X >> 1, 32);
		PropertiesSizeX = FMath::Max(MaterialSettings.TextureSize.X >> 2, 16);

		if (InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse).X != DiffuseSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse), DiffuseSizeX, DiffuseSizeX, NewSamples, true);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Diffuse, FIntPoint(DiffuseSizeX, DiffuseSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Num() && MaterialSettings.bSpecularMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular).X != PropertiesSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular), PropertiesSizeX, PropertiesSizeX, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Specular, FIntPoint(PropertiesSizeX, PropertiesSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Num() && MaterialSettings.bMetallicMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic).X != PropertiesSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic), PropertiesSizeX, PropertiesSizeX, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Metallic, FIntPoint(PropertiesSizeX, PropertiesSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Num() && MaterialSettings.bRoughnessMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness).X != PropertiesSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness), PropertiesSizeX, PropertiesSizeX, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Roughness, FIntPoint(PropertiesSizeX, PropertiesSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal).Num() && MaterialSettings.bNormalMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal).X != NormalSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal), NormalSizeX, NormalSizeX, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Normal, FIntPoint(NormalSizeX, NormalSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive).Num() && MaterialSettings.bEmissiveMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive).X != PropertiesSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive), PropertiesSizeX, PropertiesSizeX, NewSamples, true);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Emissive, FIntPoint(PropertiesSizeX, PropertiesSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity).Num() && MaterialSettings.bOpacityMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity).X != PropertiesSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity), PropertiesSizeX, PropertiesSizeX, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Opacity, FIntPoint(PropertiesSizeX, PropertiesSizeX));
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask).Num() && MaterialSettings.bOpacityMaskMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask).X != PropertiesSizeX)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask), PropertiesSizeX, PropertiesSizeX, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::OpacityMask, FIntPoint(PropertiesSizeX, PropertiesSizeX));
		}
	}
	else if (MaterialSettings.TextureSizingType == TextureSizingType_UseManualOverrideTextureSize)
	{
		if (InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse) != MaterialSettings.DiffuseTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse), MaterialSettings.DiffuseTextureSize.X, MaterialSettings.DiffuseTextureSize.Y, NewSamples, true);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Diffuse, MaterialSettings.DiffuseTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Num() && MaterialSettings.bSpecularMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular) != MaterialSettings.SpecularTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Specular).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular), MaterialSettings.SpecularTextureSize.X, MaterialSettings.SpecularTextureSize.Y, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Specular, MaterialSettings.SpecularTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Num() && MaterialSettings.bMetallicMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic) != MaterialSettings.MetallicTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic), MaterialSettings.MetallicTextureSize.X, MaterialSettings.MetallicTextureSize.Y, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Metallic, MaterialSettings.MetallicTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Num() && MaterialSettings.bRoughnessMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness) != MaterialSettings.RoughnessTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness), MaterialSettings.RoughnessTextureSize.X, MaterialSettings.RoughnessTextureSize.Y, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Roughness, MaterialSettings.RoughnessTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal).Num() && MaterialSettings.bNormalMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal) != MaterialSettings.NormalTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal), MaterialSettings.NormalTextureSize.X, MaterialSettings.NormalTextureSize.Y, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Normal).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Normal, MaterialSettings.NormalTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive).Num() && MaterialSettings.bEmissiveMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive) != MaterialSettings.EmissiveTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive), MaterialSettings.EmissiveTextureSize.X, MaterialSettings.EmissiveTextureSize.Y, NewSamples, true);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Emissive, MaterialSettings.EmissiveTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity).Num() && MaterialSettings.bOpacityMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity) != MaterialSettings.OpacityTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Opacity).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity), MaterialSettings.OpacityTextureSize.X, MaterialSettings.OpacityTextureSize.Y, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Opacity).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Opacity, MaterialSettings.OpacityTextureSize);
		}

		if (InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask).Num() && MaterialSettings.bOpacityMaskMap && InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask) != MaterialSettings.OpacityMaskTextureSize)
		{
			TArray<FColor> NewSamples;
			FImageUtils::ImageResize(InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask).X, InFlattenMaterial.GetPropertySize(EFlattenMaterialProperties::OpacityMask).Y, InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask), MaterialSettings.OpacityMaskTextureSize.X, MaterialSettings.OpacityMaskTextureSize.Y, NewSamples, false);
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask).Reset(NewSamples.Num());
			InFlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::OpacityMask).Append(NewSamples);
			InFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::OpacityMask, MaterialSettings.OpacityMaskTextureSize);
		}
	}
}

/** Computes the uniform scale from the input scales,  if one exists. */
static float GetUniformScale(const TArray<float> Scales)
{
	if (Scales.Num())
	{
		float Average = 0;
		float Mean = 0;

		for (float V : Scales)
		{
			Average += V;
		}
		Average /= (float)Scales.Num();

		for (float V : Scales)
		{
			Mean += FMath::Abs(V - Average);
		}
		Mean /= (float)Scales.Num();

		if (Mean * 15.f < Average) // If they are almost all the same
		{
			return Average;
		}
		else // Otherwise do a much more expensive test by counting the number of similar values
		{
			// Try to find a small range where 80% of values fit within.
			const int32 TryThreshold = FMath::CeilToInt(.80f * (float)Scales.Num());

			int32 NextTryDomain = Scales.Num();

			float NextTryMinV = 1024;
			for (float V : Scales)
			{
				NextTryMinV = FMath::Min<float>(V, NextTryMinV);
			}

			while (NextTryDomain >= TryThreshold) // Stop the search it is garantied to fail.
			{
				float TryMinV = NextTryMinV;
				float TryMaxV = TryMinV * 1.25f;
				int32 TryMatches = 0;
				NextTryMinV = 1024;
				NextTryDomain = 0;
				for (float V : Scales)
				{
					if (TryMinV <= V && V <= TryMaxV)
					{
						++TryMatches;
					}

					if (V > TryMinV)
					{
						NextTryMinV = FMath::Min<float>(V, NextTryMinV);
						++NextTryDomain;
					}
				}

				if (TryMatches >= TryThreshold)
				{
					return TryMinV;
				}
			}
		}
	}
	return 0;
}

uint32 GetTypeHash(const FMaterialUtilities::FExportErrorManager::FError& Error)
{
	return GetTypeHash(Error.Material);
}

bool FMaterialUtilities::FExportErrorManager::FError::operator==(const FError& Rhs) const
{
	return Material == Rhs.Material && RegisterIndex == Rhs.RegisterIndex && ErrorType == Rhs.ErrorType;
}


void FMaterialUtilities::FExportErrorManager::Register(const UMaterialInterface* Material, FName TextureName, int32 RegisterIndex, EErrorType ErrorType)
{
	if (!Material || TextureName == NAME_None) return;

	FError Error;
	Error.Material = Material->GetMaterialResource(ShaderPlatform);
	if (!Error.Material) return;
	Error.RegisterIndex = RegisterIndex;
	Error.ErrorType = ErrorType;

	FInstance Instance;
	Instance.Material = Material;
	Instance.TextureName = TextureName;

	ErrorInstances.FindOrAdd(Error).Push(Instance);
}

void FMaterialUtilities::FExportErrorManager::OutputToLog()
{
	const UMaterialInterface* CurrentMaterial = nullptr;
	int32 MaxInstanceCount = 0;
	FString TextureErrors;

	for (TMap<FError, TArray<FInstance> >::TIterator It(ErrorInstances);; ++It)
	{
		if (It && !It->Value.Num()) continue;

		// Here we pack texture list per material.
		if (!It || CurrentMaterial != It->Value[0].Material)
		{
			// Flush
			if (CurrentMaterial)
			{
				FString SimilarCount(TEXT(""));
				if (MaxInstanceCount > 1)
				{
					SimilarCount = FString::Printf(TEXT(", %d similar"), MaxInstanceCount - 1);
				}

				if (CurrentMaterial == CurrentMaterial->GetMaterial())
				{
					UE_LOGF(TextureStreamingBuild, Verbose, "Incomplete texcoord scale analysis for %ls%ls: %ls", *CurrentMaterial->GetName(), *SimilarCount, *TextureErrors);
				}
				else
				{
					UE_LOGF(TextureStreamingBuild, Verbose, "Incomplete texcoord scale analysis for %ls, UMaterial=%ls%ls: %ls", *CurrentMaterial->GetName(), *CurrentMaterial->GetMaterial()->GetName(), *SimilarCount, *TextureErrors);
				}
			}

			// Exit
			if (!It)
			{
				break;
			}

			// Start new
			CurrentMaterial = It->Value[0].Material;
			MaxInstanceCount = It->Value.Num();
			TextureErrors.Empty();
		}
		else
		{
			// Append
			MaxInstanceCount = FMath::Max<int32>(MaxInstanceCount, It->Value.Num());
		}

		const TCHAR* ErrorMsg = TEXT("Unkown Error");
		if (It->Key.ErrorType == EET_IncohorentValues)
		{
			ErrorMsg = TEXT("Incoherent");
		}
		else if (It->Key.ErrorType == EET_NoValues)
		{
			ErrorMsg = TEXT("NoValues");
		}

		TextureErrors.Append(FString::Printf(TEXT("(%s:%d,%s) "), ErrorMsg, It->Key.RegisterIndex, *It->Value[0].TextureName.ToString()));
	}
}

// Deprecated in 5.7
bool FMaterialUtilities::ExportMaterialUVDensities(UMaterialInterface* InMaterial, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, FExportErrorManager& OutErrors)
{
	return ExportMaterialUVDensities(InMaterial, QualityLevel, GetFeatureLevelShaderPlatform_Checked(FeatureLevel), OutErrors);
}

bool FMaterialUtilities::ExportMaterialUVDensities(UMaterialInterface* InMaterial, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform, FExportErrorManager& OutErrors)
{
	check(InMaterial);

	// Clear the build data.
	TArray<FMaterialTextureInfo> TextureStreamingData;
	InMaterial->SetTextureStreamingData(TextureStreamingData);

	TArray<FFloat16Color> RenderedVectors;

	TArray<UTexture*> Textures;
	TArray< TArray<int32> > Indices;
	InMaterial->GetUsedTexturesAndIndices(Textures, Indices, QualityLevel, InShaderPlatform);
	FTextureCompilingManager::Get().FinishCompilation(Textures);

	check(Textures.Num() >= Indices.Num()); // Can't have indices if no texture.

	const int32 SCALE_PRECISION = 64.f;

	int32 MaxRegisterIndex = INDEX_NONE;
	for (const TArray<int32>& TextureIndices : Indices)
	{
		for (int32 RegisterIndex : TextureIndices)
		{
			MaxRegisterIndex = FMath::Max<int32>(RegisterIndex, MaxRegisterIndex);
		}
	}

	if (MaxRegisterIndex == INDEX_NONE)
	{
		return false;
	}

	// Find the streaming texture for each material texture register index.
	TArray<UTexture2D*> RegisterIndexToTextures;
	RegisterIndexToTextures.AddZeroed(MaxRegisterIndex + 1);
	for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); ++TextureIndex)
	{
		UTexture2D* Texture2D = Cast<UTexture2D>(Textures[TextureIndex]);
		if (Texture2D) // Don't check IsStreamingTexture() yet as this could change before cooking.
		{
			for (int32 RegisterIndex : Indices[TextureIndex])
			{
				RegisterIndexToTextures[RegisterIndex] = Texture2D;
			}
		}
	}

	const int32 NumTileX = (MaxRegisterIndex / 4 + 1);
	const int32 NumTileY = TEXSTREAM_MAX_NUM_UVCHANNELS;
	FIntPoint RenderTargetSize(TEXSTREAM_TILE_RESOLUTION * NumTileX, TEXSTREAM_TILE_RESOLUTION * NumTileY);

	// Render the vectors
	{
		// The rendertarget contain factors stored in XYZW. Every X tile maps to another group : (0, 1, 2, 3), (4, 5, 6, 7), ...
		UTextureRenderTarget2D* RenderTarget = CreateRenderTarget(true, false, PF_FloatRGBA, RenderTargetSize);

		// Allocate the render output.
		RenderedVectors.Empty(RenderTargetSize.X * RenderTargetSize.Y);

		FMaterialRenderProxy* MaterialProxy = InMaterial->GetRenderProxy();
		if (!MaterialProxy)
		{
			return false;
		}

		FBox2D DummyBounds(FVector2D(0, 0), FVector2D(1, 1));
		TArray<FVector2D> EmptyTexCoords;
		FMaterialMergeData MaterialData(InMaterial, nullptr, nullptr, 0, DummyBounds, EmptyTexCoords);

		CurrentlyRendering = true;
		bool bResult = FMeshRenderer::RenderMaterialTexCoordScales(MaterialData, MaterialProxy, RenderTarget, RenderedVectors);
		CurrentlyRendering = false;

		if (!bResult)
		{
			return false;
		}
	}

	// Now compute the scale for each texture index (several indices could map to the same texture)
	for (int32 RegisterIndex = 0; RegisterIndex <= MaxRegisterIndex; ++RegisterIndex)
	{
		UTexture2D* Texture2D = RegisterIndexToTextures[RegisterIndex];
		if (!Texture2D) continue; // Only handle streaming textures

		int32 TextureTile = RegisterIndex / 4;
		int32 ComponentIndex = RegisterIndex % 4;

		bool bSuccess = false;
		bool bHadAnyValues = false;

		for (int32 CoordIndex = 0; CoordIndex < TEXSTREAM_MAX_NUM_UVCHANNELS && !bSuccess; ++CoordIndex)
		{
			TArray<float> TextureScales;
			TextureScales.Empty(TEXSTREAM_TILE_RESOLUTION * TEXSTREAM_TILE_RESOLUTION);
			for (int32 TexelX = 0; TexelX < TEXSTREAM_TILE_RESOLUTION; ++TexelX)
			{
				for (int32 TexelY = 0; TexelY < TEXSTREAM_TILE_RESOLUTION; ++TexelY)
				{
					int32 TexelIndex = TextureTile * TEXSTREAM_TILE_RESOLUTION + TexelX + (TexelY + CoordIndex * TEXSTREAM_TILE_RESOLUTION) * RenderTargetSize.X;
					FFloat16Color& Scale16 = RenderedVectors[TexelIndex];

					float TexelScale = 0;
					if (ComponentIndex == 0) TexelScale = Scale16.R.GetFloat();
					if (ComponentIndex == 1) TexelScale = Scale16.G.GetFloat();
					if (ComponentIndex == 2) TexelScale = Scale16.B.GetFloat();
					if (ComponentIndex == 3) TexelScale = Scale16.A.GetFloat();

					// Quantize scale to converge faster in the TryLogic
					TexelScale = FMath::RoundToFloat(TexelScale * SCALE_PRECISION) / SCALE_PRECISION;

					if (TexelScale > 0 && TexelScale < TEXSTREAM_INITIAL_GPU_SCALE)
					{
						TextureScales.Push(TexelScale);
					}
				}
			}

			const float SamplingScale = GetUniformScale(TextureScales);
			if (SamplingScale > 0)
			{
				FMaterialTextureInfo TextureInfo;
				TextureInfo.SamplingScale = SamplingScale;
				TextureInfo.UVChannelIndex = CoordIndex;
				TextureInfo.TextureReference = FSoftObjectPath(Texture2D);
				TextureInfo.TextureIndex = RegisterIndex;
				TextureStreamingData.Add(TextureInfo);
				bSuccess = true;
			}
			else if (TextureScales.Num())
			{
				bHadAnyValues = true;
			}
		}

		// If we couldn't find the scale, then output a warning detailing which index, texture, material is having an issue.
		if (!bSuccess)
		{
			OutErrors.Register(InMaterial, Texture2D->GetFName(), RegisterIndex, bHadAnyValues ? FExportErrorManager::EErrorType::EET_IncohorentValues : FExportErrorManager::EErrorType::EET_NoValues);
		}
	}

	// Update to the final data.
	InMaterial->SetTextureStreamingData(TextureStreamingData);

	return true;
}

UTextureRenderTarget2D* FMaterialUtilities::CreateRenderTarget(bool bInForceLinearGamma, bool bNormalMap, EPixelFormat InPixelFormat, FIntPoint& InTargetSize)
{
	const FLinearColor ClearColour = bNormalMap ? FLinearColor(0.0f, 0.0f, 0.0f, 0.0f) : FLinearColor(1.0f, 0.0f, 1.0f, 0.0f);
	
	// Find any pooled render target with suitable properties.
	for (int32 RTIndex = 0; RTIndex < RenderTargetPool.Num(); RTIndex++)
	{
		UTextureRenderTarget2D* RenderTarget = RenderTargetPool[RTIndex];
		if (RenderTarget->SizeX == InTargetSize.X &&
			RenderTarget->SizeY == InTargetSize.Y &&
			RenderTarget->OverrideFormat == InPixelFormat &&
			RenderTarget->bForceLinearGamma == bInForceLinearGamma &&
			RenderTarget->ClearColor == ClearColour )
		{
			return RenderTarget;
		}
	}

	// Not found - create a new one.
	UTextureRenderTarget2D* NewRenderTarget = NewObject<UTextureRenderTarget2D>();
	check(NewRenderTarget);
	NewRenderTarget->AddToRoot();
	NewRenderTarget->ClearColor = ClearColour;
	NewRenderTarget->TargetGamma = 0.0f;
	NewRenderTarget->InitCustomFormat(InTargetSize.X, InTargetSize.Y, InPixelFormat, bInForceLinearGamma);

	RenderTargetPool.Add(NewRenderTarget);
	return NewRenderTarget;
}

void FMaterialUtilities::ClearRenderTargetPool()
{
	if (CurrentlyRendering)
	{
		// Just in case - if garbage collection will happen during rendering, don't allow to GC used render target.
		return;
	}

	// Allow garbage collecting of all render targets.
	for (int32 RTIndex = 0; RTIndex < RenderTargetPool.Num(); RTIndex++)
	{
		RenderTargetPool[RTIndex]->RemoveFromRoot();
	}
	RenderTargetPool.Empty();
}

void FMaterialUtilities::OptimizeSampleArray(TArray<FColor>& InSamples, FIntPoint& InSampleSize)
{
	if (InSamples.Num() > 1)
	{
		TArray<FColor> Colors;

		for (const FColor& Sample : InSamples)
		{
			if (Colors.AddUnique(Sample) != 0)
			{
				break;
			}
		}

		if (Colors.Num() == 1)
		{
			InSamples.Empty(1);
			InSamples.Add(Colors[0]);
			InSampleSize = FIntPoint(1, 1);
		}
	}	
}

FExportMaterialProxyCache::FExportMaterialProxyCache()
{
	FMemory::Memzero(Proxies);
}

FExportMaterialProxyCache::~FExportMaterialProxyCache()
{
	Release();
}

void FExportMaterialProxyCache::Release()
{
	for (int32 PropertyIndex = 0; PropertyIndex < UE_ARRAY_COUNT(Proxies); PropertyIndex++)
	{
		FMaterialRenderProxy* Proxy = Proxies[PropertyIndex];
		if (Proxy)
		{
			delete Proxy;
			Proxies[PropertyIndex] = nullptr;
		}
	}
}


void FMaterialUtilities::DetermineMaterialImportance(const TArray<UMaterialInterface*>& InMaterials, TArray<float>& OutImportance)
{
	TArray<int32> MaterialImportance;
	int32 SummedSize = 0;
	for (UMaterialInterface* Material : InMaterials)
	{
		TArray<UObject*> UsedTextures(Material->GetReferencedTextures());
		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Material))
		{
			for (const FTextureParameterValue& TextureParameter : MaterialInstance->TextureParameterValues)
			{
				if (TextureParameter.ParameterValue != nullptr)
				{
					UsedTextures.Add(TextureParameter.ParameterValue);
				}
			}
		}
		int32 MaxSize = 64 * 64;
		for (UObject* Texture : UsedTextures)
		{
			if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
			{
				const int32 MaxResMipBias = Texture2D->GetNumMips() - Texture2D->GetNumMipsAllowed(true);
				const int32 MaxResSize = FMath::Max<int32>(Texture2D->GetSizeX() >> MaxResMipBias, 1) * FMath::Max<int32>(Texture2D->GetSizeY() >> MaxResMipBias, 1);
				MaxSize = FMath::Max<int32>(MaxSize, MaxResSize);
			}
		}
		
		MaterialImportance.Add(MaxSize);
		SummedSize += MaxSize;
	}
	float WeightPerPixel = 1.0f / SummedSize;
	for (int32 MaterialIndex = 0; MaterialIndex < InMaterials.Num(); ++MaterialIndex)
	{
		OutImportance.Add((float)MaterialImportance[MaterialIndex] * WeightPerPixel);
	}	
}

void FMaterialUtilities::GeneratedBinnedTextureSquares(const FVector2D DestinationSize, TArray<float>& InTexureWeights, TArray<FBox2D>& OutGeneratedBoxes)
{
	typedef FBox2D FTextureArea;
	struct FWeightedTexture
	{
		FTextureArea Area;
		int32 TextureIndex;
		float Weight;
	};

	TArray<FWeightedTexture> WeightedTextures;
	const float TotalArea = DestinationSize.X * DestinationSize.Y;
	// Generate textures with their size calculated according to their weight
	for (int32 WeightIndex = 0; WeightIndex < InTexureWeights.Num(); ++WeightIndex)
	{
		const float Weight = InTexureWeights[WeightIndex];
		FWeightedTexture Texture;
		float TextureSize = FMath::Sqrt(TotalArea*Weight);
		Texture.Area = FTextureArea( FVector2D(0.0f,0.0f), FVector2D(TextureSize, TextureSize));
		Texture.TextureIndex = WeightIndex;
		Texture.Weight = Weight;
		WeightedTextures.Add(Texture);
	}

	// Sort textures by their weight (high to low) which influences the insert order
	WeightedTextures.Sort([](const FWeightedTexture& One, const FWeightedTexture& Two) { return One.Weight > Two.Weight; });
		
	TArray<FWeightedTexture> InsertedTextures;
	typedef FBox2D FUnusedArea;
	TArray<FUnusedArea> UnusedAreas;

	bool bSuccess = true;
	do 
	{
		// Reset state
		bSuccess = true;
		UnusedAreas.Empty();
		InsertedTextures.Empty();
		FUnusedArea StartArea(FVector2D(0, 0), DestinationSize);
		UnusedAreas.Add(StartArea);

		for (const FWeightedTexture& Texture : WeightedTextures)
		{
			int32 BestAreaIndex = -1;
			float RemainingArea = FLT_MAX;
			FVector2D TextureSize = Texture.Area.GetSize();
			float TextureSurface = TextureSize.X * TextureSize.Y;

			// Find best area to insert this texture in (determined by tightest fit)
			for (int32 AreaIndex = 0; AreaIndex < UnusedAreas.Num(); ++AreaIndex)
			{
				const FUnusedArea& UnusedArea = UnusedAreas[AreaIndex];
				if (UnusedArea.GetSize().ComponentwiseAllGreaterOrEqual(TextureSize))
				{
					const float Remainder = UnusedArea.GetArea() - TextureSurface;
					if (Remainder < RemainingArea && Remainder >= 0)
					{
						BestAreaIndex = AreaIndex;
						RemainingArea = Remainder;
					}
				}
			}

			// Insert the texture in case we found an appropriate area
			if (BestAreaIndex != -1)
			{
				FUnusedArea& UnusedArea = UnusedAreas[BestAreaIndex];
				FVector2D UnusedSize = UnusedArea.GetSize();

				// Push back texture
				FWeightedTexture WeightedTexture;
				WeightedTexture.Area = FTextureArea(UnusedArea.Min, UnusedArea.Min + TextureSize);
				WeightedTexture.TextureIndex = Texture.TextureIndex;
				InsertedTextures.Add(WeightedTexture);
				
				// Generate two new resulting unused areas from splitting up the result
				/*
					___________
					|	  |   |
					|	  | V |
					|_____|   |
					|  H  |   |
					|_____|___|
				*/
				FUnusedArea HorizontalArea, VerticalArea;
				HorizontalArea.Min.X = UnusedArea.Min.X;
				HorizontalArea.Min.Y = UnusedArea.Min.Y + TextureSize.Y;
				HorizontalArea.Max.X = HorizontalArea.Min.X + TextureSize.X;
				HorizontalArea.Max.Y = HorizontalArea.Min.Y + (UnusedSize.Y - TextureSize.Y);

				VerticalArea.Min.X = UnusedArea.Min.X + TextureSize.X;
				VerticalArea.Min.Y = UnusedArea.Min.Y;
				VerticalArea.Max.X = VerticalArea.Min.X + (UnusedSize.X - TextureSize.X);
				VerticalArea.Max.Y = UnusedSize.Y;

				// Append valid new areas to list (replace original one with either one of the new ones)
				const bool bValidHorizontal = HorizontalArea.GetArea() > 0.0f;
				const bool bValidVertical = VerticalArea.GetArea() > 0.0f;
				if (bValidVertical && bValidHorizontal)
				{
					UnusedAreas[BestAreaIndex] = HorizontalArea;
					UnusedAreas.Add(VerticalArea);
				}
				else if (bValidVertical)
				{
					UnusedAreas[BestAreaIndex] = VerticalArea;
				}
				else if (bValidHorizontal)
				{
					UnusedAreas[BestAreaIndex] = HorizontalArea;
				}
				else
				{
					// Make sure we remove the area entry
					UnusedAreas.RemoveAtSwap(BestAreaIndex);
				}
			}
			else
			{
				bSuccess = false;
				break;
			}
		}

		// This means we failed to find a fit, in this case we resize the textures and try again until we find one
		if (bSuccess == false)
		{
			for (FWeightedTexture& Texture : WeightedTextures)
			{
				Texture.Area.Max *= .99f;
			}
		}
	} while (!bSuccess);

	// Now generate boxes
	OutGeneratedBoxes.Empty(InTexureWeights.Num());
	OutGeneratedBoxes.AddZeroed(InTexureWeights.Num());

	// Generate boxes according to the inserted textures
	for (const FWeightedTexture& Texture : InsertedTextures)
	{
		FBox2D& Box = OutGeneratedBoxes[Texture.TextureIndex];
		Box = Texture.Area;
	}
}

float FMaterialUtilities::ComputeRequiredTexelDensityFromScreenSize(const float InScreenSize, float InWorldSpaceRadius)
{
	static const float ScreenX = 1920;

	float WorldSizeCM = InWorldSpaceRadius * 2;
	float WorldSizeMeter = WorldSizeCM / 100;

	float ScreenSizePercent = InScreenSize;
	float ScreenSizePixel = ScreenSizePercent * ScreenX;

	float TexelDensityPerMeter = ScreenSizePixel / WorldSizeMeter;

	return TexelDensityPerMeter;
}

float FMaterialUtilities::ComputeRequiredTexelDensityFromDrawDistance(const float InDrawDistance, float InWorldSpaceRadius)
{
	// Generate a projection matrix.
	static const float ScreenX = 1920;
	static const float ScreenY = 1080;
	static const float HalfFOVRad = FMath::DegreesToRadians(45.0f);
	static const FMatrix ProjectionMatrix = FPerspectiveMatrix(HalfFOVRad, ScreenX, ScreenY, 0.01f);

	float WorldSizeCM = InWorldSpaceRadius * 2;
	float WorldSizeMeter = WorldSizeCM / 100;

	float ScreenSizePercent = ComputeBoundsScreenSize(FVector::ZeroVector, InWorldSpaceRadius, FVector(0.0f, 0.0f, InDrawDistance), ProjectionMatrix);
	float ScreenSizePixel = ScreenSizePercent * ScreenX;

	float TexelDensityPerMeter = ScreenSizePixel / WorldSizeMeter;

	return TexelDensityPerMeter;
}

static int32 ComputeTextureSizeFromTexelRatio(const double TexelRatio, const double TargetTexelDensity)
{
	// Compute the perfect texture size that would get us to our texture density
	// Also compute the nearest power of two sizes (below and above our target)
	const int32 SizePerfect = FMath::CeilToInt(TargetTexelDensity / TexelRatio);
	const int32 SizeHi = FMath::RoundUpToPowerOfTwo(SizePerfect);
	const int32 SizeLo = SizeHi >> 1;

	// Compute the texel density we achieve with these two texture sizes
	const double TexelDensityLo = SizeLo * TexelRatio;
	const double TexelDensityHi = SizeHi * TexelRatio;

	// Select best match between low & high res textures.
	const double TexelDensityLoDiff = TargetTexelDensity - TexelDensityLo;
	const double TexelDensityHiDiff = TexelDensityHi - TargetTexelDensity;
	const int32 BestTextureSize = TexelDensityLoDiff < TexelDensityHiDiff ? SizeLo : SizeHi;

	return BestTextureSize;
}

int32 FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(const UE::Geometry::FDynamicMesh3& Mesh, float InTargetTexelDensity)
{
	const UE::Geometry::FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->PrimaryUV();

	double Mesh3DArea = 0;
	double MeshUVArea = 0;

	// If no UVs, assume perfect UV space usage.
	const bool bHasUVs = UVOverlay != nullptr;
	if (!bHasUVs)
	{
		MeshUVArea = 1.0;
	}

	for (int TriangleID : Mesh.TriangleIndicesItr())
	{
		// World space area
		Mesh3DArea += Mesh.GetTriArea(TriangleID);

		// UV space area
		if (bHasUVs)
		{
			UE::Geometry::FIndex3i UVVertices = UVOverlay->GetTriangle(TriangleID);
			UE::Geometry::FTriangle2d TriangleUV = UE::Geometry::FTriangle2d(
				(FVector2d)UVOverlay->GetElement(UVVertices.A),
				(FVector2d)UVOverlay->GetElement(UVVertices.B),
				(FVector2d)UVOverlay->GetElement(UVVertices.C));

			MeshUVArea += TriangleUV.Area();
		}
	}

	return GetTextureSizeFromTargetTexelDensity(Mesh3DArea, MeshUVArea, InTargetTexelDensity);
}

int32 FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(const FMeshDescription& InMesh, float InTargetTexelDensity)
{
	FStaticMeshConstAttributes Attributes(InMesh);

	TVertexAttributesConstRef<FVector3f> Positions = Attributes.GetVertexPositions();
	TUVAttributesConstRef<FVector2f> UVs = Attributes.GetUVCoordinates(0);

	double Mesh3DArea = 0;
	double MeshUVArea = 0;

	// If no UVs, assume perfect UV space usage.
	const bool bHasUVs = UVs.IsValid() && UVs.GetNumElements() != 0;
	if (!bHasUVs)
	{
		MeshUVArea = 1.0;
	}

	for (const FTriangleID TriangleID : InMesh.Triangles().GetElementIDs())
	{
		// World space area
		TArrayView<const FVertexID> TriVertices = InMesh.GetTriangleVertices(TriangleID);
		Mesh3DArea += UE::Geometry::VectorUtil::Area(Positions[TriVertices[0]], Positions[TriVertices[1]], Positions[TriVertices[2]]);

		// UV space area
		if (bHasUVs)
		{
			TArrayView<const FUVID> TriUVs = InMesh.GetTriangleUVIndices(TriangleID);
			MeshUVArea += UE::Geometry::VectorUtil::Area(UVs[0], UVs[1], UVs[2]);
		}
	}

	return GetTextureSizeFromTargetTexelDensity(Mesh3DArea, MeshUVArea, InTargetTexelDensity);
}

int32 FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(double InMesh3DArea, double InMeshUVArea, double InTargetTexelDensity)
{
	double TexelRatio = FMath::Sqrt(InMeshUVArea / InMesh3DArea) * 100;

	static const int32 MinTextureSize = 16;
	static const int32 MaxTextureSize = 8192;

	int32 TextureSize = ComputeTextureSizeFromTexelRatio(TexelRatio, InTargetTexelDensity);
	if (TextureSize > MaxTextureSize)
	{
		UE_LOGF(LogMaterialUtilities, Warning, "Mesh would require %d x %d textures, clamping down to maximum (%d x %d)", TextureSize, TextureSize, MaxTextureSize, MaxTextureSize);
	}

	return FMath::Clamp(TextureSize, MinTextureSize, MaxTextureSize);
}
