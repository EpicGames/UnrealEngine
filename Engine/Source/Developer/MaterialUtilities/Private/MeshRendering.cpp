// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshRendering.cpp: Mesh rendering implementation.
=============================================================================*/

#include "MeshRendering.h"
#include "EngineDefines.h"
#include "ShowFlags.h"
#include "RHI.h"
#include "RenderResource.h"
#include "HitProxies.h"
#include "RenderingThread.h"
#include "VertexFactory.h"
#include "TextureResource.h"
#include "PackedNormal.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/App.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialUtilities.h"
#include "MeshBuilderOneFrameResources.h"
#include "Misc/FileHelper.h"
#include "StaticMeshAttributes.h"
#include "SceneView.h"
#include "MeshBatch.h"
#include "CanvasItem.h"
#include "CanvasRender.h"
#include "LocalVertexFactory.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "MeshPassProcessor.h"
#include "RHIStaticStates.h"
#include "RendererInterface.h"
#include "EngineModule.h"
#include "LightMapHelpers.h"
#include "Async/ParallelFor.h"
#include "DynamicMeshBuilder.h"
#include "MaterialBakingHelpers.h"
#include "MaterialShared.h"

#define SHOW_WIREFRAME_MESH 0
#define SAVE_INTERMEDIATE_TEXTURES 0

class FMeshRenderInfo : public FLightCacheInterface
{
public:
	FMeshRenderInfo(const FLightMap* InLightMap, const FShadowMap* InShadowMap, FUniformBufferRHIRef Buffer)
		: FLightCacheInterface()
	{
		SetLightMap(InLightMap);
		SetShadowMap(InShadowMap);
		SetPrecomputedLightingBuffer(Buffer);
	}

	virtual FLightInteraction GetInteraction(const class FLightSceneProxy* LightSceneProxy) const override
	{
		return LIT_CachedLightMap;
	}
};

/**
* Canvas render item enqueued into renderer command list.
*/
class FMeshMaterialRenderItem2 : public FCanvasBaseRenderItem
{
public:
	FMeshMaterialRenderItem2(FSceneViewFamily* InViewFamily, const FMeshDescription* InMesh, const FSkeletalMeshLODRenderData* InLODData, int32 LightMapIndex, int32 InMaterialIndex, const FBox2D& InTexcoordBounds, const TArray<FVector2D>& InTexCoords, const FVector2D& InSize, const FMaterialRenderProxy* InMaterialRenderProxy, const FCanvas::FTransformEntry& InTransform /*= FCanvas::FTransformEntry(FMatrix::Identity)*/, FLightMapRef LightMap, FShadowMapRef ShadowMap, FUniformBufferRHIRef Buffer) : Data(new FRenderData(
		InViewFamily,
		InMesh,
		InLODData,
		LightMapIndex,
		InMaterialIndex,
		InTexcoordBounds,
		InTexCoords,
		InSize,
		InMaterialRenderProxy,
		InTransform,
		new FMeshRenderInfo(LightMap, ShadowMap, Buffer)))
	{
	}

	~FMeshMaterialRenderItem2()
	{
	}

private:
	class FRenderData
	{
	public:
		FRenderData(
			FSceneViewFamily* InViewFamily,
			const FMeshDescription* InMesh,
			const FSkeletalMeshLODRenderData* InLODData,
			int32 InLightMapIndex,
			int32 InMaterialIndex,
			const FBox2D& InTexcoordBounds,
			const TArray<FVector2D>& InTexCoords,
			const FVector2D& InSize,
			const FMaterialRenderProxy* InMaterialRenderProxy = nullptr,
			const FCanvas::FTransformEntry& InTransform = FCanvas::FTransformEntry(FMatrix::Identity),
			FLightCacheInterface* InLCI = nullptr)
			: ViewFamily(InViewFamily)
			, StaticMesh(InMesh)
			, SkeletalMesh(InLODData)
			, LightMapIndex(InLightMapIndex)
			, MaterialIndex(InMaterialIndex)
			, TexcoordBounds(InTexcoordBounds)
			, TexCoords(InTexCoords)
			, Size(InSize)
			, MaterialRenderProxy(InMaterialRenderProxy)
			, Transform(InTransform)
			, LCI(InLCI)

		{}
		FSceneViewFamily* ViewFamily;
		const FMeshDescription* StaticMesh;
		const FSkeletalMeshLODRenderData* SkeletalMesh;
		int32 LightMapIndex;
		int32 MaterialIndex;
		FBox2D TexcoordBounds;
		const TArray<FVector2D>& TexCoords;
		FVector2D Size;
		const FMaterialRenderProxy* MaterialRenderProxy;
		
		FCanvas::FTransformEntry Transform;
		FLightCacheInterface* LCI;
	};
	FRenderData* Data;
public:

	static void EnqueueMaterialRender(class FCanvas* InCanvas, FSceneViewFamily* InViewFamily, const FMeshDescription* InMesh, const FSkeletalMeshLODRenderData* InLODRenderData, int32 LightMapIndex, int32 InMaterialIndex, const FBox2D& InTexcoordBounds, const TArray<FVector2D>& InTexCoords, const FVector2D& InSize, const FMaterialRenderProxy* InMaterialRenderProxy, FLightMapRef LightMap, FShadowMapRef ShadowMap, FUniformBufferRHIRef Buffer)
	{
		// get sort element based on the current sort key from top of sort key stack
		FCanvas::FCanvasSortElement& SortElement = InCanvas->GetSortElement(InCanvas->TopDepthSortKey());
		// get the current transform entry from top of transform stack
		const FCanvas::FTransformEntry& TopTransformEntry = InCanvas->GetTransformStack().Top();
		// create a render batch
		FMeshMaterialRenderItem2* RenderBatch = new FMeshMaterialRenderItem2(
			InViewFamily,
			InMesh,
			InLODRenderData,
			LightMapIndex,
			InMaterialIndex,
			InTexcoordBounds,
			InTexCoords,
			InSize,
			InMaterialRenderProxy,
			TopTransformEntry,
			LightMap,
			ShadowMap, 
			Buffer);
		SortElement.RenderBatchArray.Add(RenderBatch);
	}

	static int32 FillStaticMeshData(bool bDuplicateTris, const FMeshDescription& RawMesh, FRenderData& Data, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
	{
		// count triangles for selected material
		int32 NumTris = 0;
		for (const FTriangleID TriangleID : RawMesh.Triangles().GetElementIDs())
		{
			const FPolygonGroupID PolygonGroupID = RawMesh.GetTrianglePolygonGroup(TriangleID);
			if (PolygonGroupID.GetValue() == Data.MaterialIndex)
			{
				NumTris++;
			}
		}
		if (NumTris == 0)
		{
			// there's nothing to do here
			return 0;
		}

		FStaticMeshConstAttributes Attributes(RawMesh);
		TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();

		int32 NumVerts = NumTris * 3;

		// reserve renderer data
		OutVerts.Empty(NumVerts);
		OutIndices.Empty(bDuplicateTris ? NumVerts * 2 : NumVerts);

		float U = Data.TexcoordBounds.Min.X;
		float V = Data.TexcoordBounds.Min.Y;
		float SizeU = Data.TexcoordBounds.Max.X - Data.TexcoordBounds.Min.X;
		float SizeV = Data.TexcoordBounds.Max.Y - Data.TexcoordBounds.Min.Y;
		float ScaleX = (SizeU != 0) ? Data.Size.X / SizeU : 1.0;
		float ScaleY = (SizeV != 0) ? Data.Size.Y / SizeV : 1.0;

		// count number of texture coordinates for this mesh
		int32 NumTexcoords = FMath::Min(VertexInstanceUVs.GetNumChannels(), (int32)MAX_STATIC_TEXCOORDS);

		// check if we should use NewUVs or original UV set
		bool bUseNewUVs = Data.TexCoords.Num() > 0;
		if (bUseNewUVs)
		{
			check(Data.TexCoords.Num() == VertexInstanceUVs.GetNumElements());
			ScaleX = Data.Size.X;
			ScaleY = Data.Size.Y;
		}

		// add vertices
		int32 VertIndex = 0;
		int32 FaceIndex = 0;
		for (const FTriangleID TriangleID : RawMesh.Triangles().GetElementIDs())
		{
			const FPolygonGroupID PolygonGroupID = RawMesh.GetTrianglePolygonGroup(TriangleID);

			if (PolygonGroupID.GetValue() == Data.MaterialIndex)
			{
				for (int32 Corner = 0; Corner < 3; Corner++)
				{
					const int32 SrcVertIndex = FaceIndex * 3 + Corner;
					const FVertexInstanceID SrcVertexInstanceID = RawMesh.GetTriangleVertexInstance(TriangleID, Corner);
					const FVertexID SrcVertexID = RawMesh.GetVertexInstanceVertex(SrcVertexInstanceID);

					// add vertex
					FDynamicMeshVertex* Vert = new(OutVerts)FDynamicMeshVertex();
					if (!bUseNewUVs)
					{
						// compute vertex position from original UV
						const FVector2f& UV = VertexInstanceUVs.Get(SrcVertexInstanceID, 0);
						Vert->Position.Set((UV.X - U) * ScaleX, (UV.Y - V) * ScaleY, 0);
					}
					else
					{
						const FVector2D& UV = Data.TexCoords[SrcVertIndex];
						Vert->Position.Set(UV.X * ScaleX, UV.Y * ScaleY, 0);
					}
					FVector3f TangentX = VertexInstanceTangents[SrcVertexInstanceID];
					FVector3f TangentZ = VertexInstanceNormals[SrcVertexInstanceID];
					FVector3f TangentY = FVector3f::CrossProduct(TangentZ, TangentX).GetSafeNormal() * VertexInstanceBinormalSigns[SrcVertexInstanceID];
					Vert->SetTangents(TangentX, TangentY, TangentZ);
					for (int32 TexcoordIndex = 0; TexcoordIndex < NumTexcoords; TexcoordIndex++)
					{
						Vert->TextureCoordinate[TexcoordIndex] = VertexInstanceUVs.Get(SrcVertexInstanceID, TexcoordIndex);
					}

					// Store original vertex positions in texture coordinate data
					Vert->TextureCoordinate[6].X = VertexPositions[SrcVertexID].X;
					Vert->TextureCoordinate[6].Y = VertexPositions[SrcVertexID].Y;
					Vert->TextureCoordinate[7].X = VertexPositions[SrcVertexID].Z;

					Vert->Color = FLinearColor(VertexInstanceColors[SrcVertexInstanceID]).ToFColor(true);
					// add index
					OutIndices.Add(VertIndex);
					VertIndex++;
				}
				if (bDuplicateTris)
				{
					// add the same triangle with opposite vertex order
					OutIndices.Add(VertIndex - 3);
					OutIndices.Add(VertIndex - 1);
					OutIndices.Add(VertIndex - 2);
				}
			}
			FaceIndex++;
		}
		return NumTris;
	}

	static int32 FillSkeletalMeshData(bool bDuplicateTris, const FSkeletalMeshLODRenderData& LODData, FRenderData& Data, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
	{
		TArray<uint32> IndexData;
		LODData.MultiSizeIndexContainer.GetIndexBuffer(IndexData);

		int32 NumTris = 0;
		int32 NumVerts = 0;

		const int32 SectionCount = LODData.NumNonClothingSections();

		// count triangles and vertices for selected material
		for (int32 SectionIndex = 0; SectionIndex < SectionCount; SectionIndex++)
		{
			const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];
			if (Section.MaterialIndex == Data.MaterialIndex)
			{
				NumTris += Section.NumTriangles;
				NumVerts += Section.NumVertices;
			}
		}

		if (NumTris == 0)
		{
			// there's nothing to do here
			return 0;
		}

		bool bUseNewUVs = Data.TexCoords.Num() > 0;

		if (bUseNewUVs)
		{
			// we should split all merged vertices because UVs are prepared per-corner, i.e. has
			// (NumTris * 3) vertices
			NumVerts = NumTris * 3;
		}
		
		// reserve renderer data
		OutVerts.Empty(NumVerts);
		OutIndices.Empty(bDuplicateTris ? NumVerts * 2 : NumVerts);

		float U = Data.TexcoordBounds.Min.X;
		float V = Data.TexcoordBounds.Min.Y;
		float SizeU = Data.TexcoordBounds.Max.X - Data.TexcoordBounds.Min.X;
		float SizeV = Data.TexcoordBounds.Max.Y - Data.TexcoordBounds.Min.Y;
		float ScaleX = (SizeU != 0) ? Data.Size.X / SizeU : 1.0;
		float ScaleY = (SizeV != 0) ? Data.Size.Y / SizeV : 1.0;
		uint32 DefaultColor = FColor::White.DWColor();

		int32 NumTexcoords = LODData.GetNumTexCoords();

		// check if we should use NewUVs or original UV set
		if (bUseNewUVs)
		{
			ScaleX = Data.Size.X;
			ScaleY = Data.Size.Y;
		}

		// add vertices
		if (!bUseNewUVs)
		{
			// Use original UV from mesh, render indexed mesh as indexed mesh.

			uint32 FirstVertex = 0;
			uint32 OutVertexIndex = 0;

			for (int32 SectionIndex = 0; SectionIndex < SectionCount; SectionIndex++)
			{
				const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

				const int32 NumVertsInSection = Section.NumVertices;

				if (Section.MaterialIndex == Data.MaterialIndex)
				{
					// offset to remap source mesh vertex index to destination vertex index
					int32 IndexOffset = FirstVertex - OutVertexIndex;

					// copy vertices
					int32 SrcVertIndex = FirstVertex;
					for (int32 VertIndex = 0; VertIndex < NumVertsInSection; VertIndex++)
					{
						FDynamicMeshVertex* DstVert = new(OutVerts)FDynamicMeshVertex();

						// compute vertex position from original UV
						const FVector2f UV = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(SrcVertIndex, 0);
						DstVert->Position.Set((UV.X - U) * ScaleX, (UV.Y - V) * ScaleY, 0);

						DstVert->TangentX = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(SrcVertIndex);
						DstVert->TangentZ = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(SrcVertIndex);

						for (int32 TexcoordIndex = 0; TexcoordIndex < NumTexcoords; TexcoordIndex++)
						{
							DstVert->TextureCoordinate[TexcoordIndex] = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(SrcVertIndex, TexcoordIndex);
						}

						DstVert->Color = LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(SrcVertIndex);

						SrcVertIndex++;
						OutVertexIndex++;
					}

					// copy indices
					int32 Index = Section.BaseIndex;
					for (uint32 TriIndex = 0; TriIndex < Section.NumTriangles; TriIndex++)
					{
						uint32 Index0 = IndexData[Index++] - IndexOffset;
						uint32 Index1 = IndexData[Index++] - IndexOffset;
						uint32 Index2 = IndexData[Index++] - IndexOffset;
						OutIndices.Add(Index0);
						OutIndices.Add(Index1);
						OutIndices.Add(Index2);
						if (bDuplicateTris)
						{
							// add the same triangle with opposite vertex order
							OutIndices.Add(Index0);
							OutIndices.Add(Index2);
							OutIndices.Add(Index1);
						}
					}
				}
				FirstVertex += NumVertsInSection;
			}
		}
		else // bUseNewUVs
		{
			// Use external UVs. These UVs are prepared per-corner, so we should convert indexed mesh to non-indexed, without
			// sharing of vertices between triangles.

			uint32 OutVertexIndex = 0;

			for (int32 SectionIndex = 0; SectionIndex < SectionCount; SectionIndex++)
			{
				const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];

				if (Section.MaterialIndex == Data.MaterialIndex)
				{
					// copy vertices
					int32 LastIndex = Section.BaseIndex + Section.NumTriangles * 3;
					for (int32 Index = Section.BaseIndex; Index < LastIndex; Index += 3)
					{
						for (int32 Corner = 0; Corner < 3; Corner++)
						{
							int32 CornerIndex = Index + Corner;
							int32 SrcVertIndex = IndexData[CornerIndex];
							FDynamicMeshVertex* DstVert = new(OutVerts)FDynamicMeshVertex();

							const FVector2f UV = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(SrcVertIndex, 0);
							DstVert->Position.Set(UV.X * ScaleX, UV.Y * ScaleY, 0);

							DstVert->TangentX = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentX(SrcVertIndex);
							DstVert->TangentZ = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(SrcVertIndex);

							for (int32 TexcoordIndex = 0; TexcoordIndex < NumTexcoords; TexcoordIndex++)
							{
								DstVert->TextureCoordinate[TexcoordIndex] = LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetVertexUV(SrcVertIndex, TexcoordIndex);
							}

							DstVert->Color = LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(SrcVertIndex);

							OutIndices.Add(OutVertexIndex);
							OutVertexIndex++;
						}
						if (bDuplicateTris)
						{
							// add the same triangle with opposite vertex order
							OutIndices.Add(OutVertexIndex - 3);
							OutIndices.Add(OutVertexIndex - 1);
							OutIndices.Add(OutVertexIndex - 2);
						}
					}
				}
			}
		}

		return NumTris;
	}

	static int32 FillQuadData(FRenderData& Data, TArray<FDynamicMeshVertex>& OutVerts, TArray<uint32>& OutIndices)
	{
		OutVerts.Empty(4);
		OutIndices.Empty(6);

		float U = Data.TexcoordBounds.Min.X;
		float V = Data.TexcoordBounds.Min.Y;
		float SizeU = Data.TexcoordBounds.Max.X - Data.TexcoordBounds.Min.X;
		float SizeV = Data.TexcoordBounds.Max.Y - Data.TexcoordBounds.Min.Y;
		float ScaleX = (SizeU != 0) ? Data.Size.X / SizeU : 1.0;
		float ScaleY = (SizeV != 0) ? Data.Size.Y / SizeV : 1.0;

		// add vertices
		for (int32 VertIndex = 0; VertIndex < 4; VertIndex++)
		{
			FDynamicMeshVertex* Vert = new(OutVerts)FDynamicMeshVertex();

			int X = VertIndex & 1;
			int Y = (VertIndex >> 1) & 1;

			Vert->Position.Set(ScaleX * X, ScaleY * Y, 0);
			Vert->SetTangents(FVector3f(1, 0, 0), FVector3f(0, 1, 0), FVector3f(0, 0, 1));
			FMemory::Memzero(&Vert->TextureCoordinate, sizeof(Vert->TextureCoordinate));
			Vert->TextureCoordinate[0].Set(U + SizeU * X, V + SizeV * Y);
			Vert->Color = FColor::White;
		}

		// add indices
		static const uint32 Indices[6] = { 0, 2, 1, 2, 3, 1 };
		OutIndices.Append(Indices, 6);

		return 2;
	}

	static void RenderMaterial(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const class FSceneView& View, FRenderData& Data)
	{
		// Check if material is TwoSided - single-sided materials should be rendered with normal and reverse
		// triangle corner orders, to avoid problems with inside-out meshes or mesh parts. Note:
		// FExportMaterialProxy::GetMaterial() (which is really called here) ignores 'InFeatureLevel' parameter.
		const bool bIsMaterialTwoSided = Data.MaterialRenderProxy->GetIncompleteMaterialWithFallback(GMaxRHIFeatureLevel).IsTwoSided();

		TArray<FDynamicMeshVertex> Verts;
		TArray<uint32> Indices;

		int32 NumTris = 0;
		if (Data.StaticMesh != nullptr)
		{
			check(Data.SkeletalMesh == nullptr)
				NumTris = FillStaticMeshData(!bIsMaterialTwoSided, *Data.StaticMesh, Data, Verts, Indices);
		}
		else if (Data.SkeletalMesh != nullptr)
		{
			NumTris = FillSkeletalMeshData(!bIsMaterialTwoSided, *Data.SkeletalMesh, Data, Verts, Indices);
		}
		else
		{
			// both are null, use simple rectangle
			NumTris = FillQuadData(Data, Verts, Indices);
		}
		if (NumTris == 0)
		{
			// there's nothing to do here
			return;
		}

		uint32 LightMapCoordinateIndex = (uint32)Data.LightMapIndex;
		LightMapCoordinateIndex = LightMapCoordinateIndex < MAX_STATIC_TEXCOORDS ? LightMapCoordinateIndex : MAX_STATIC_TEXCOORDS - 1;

		FDynamicMeshBuilder DynamicMeshBuilder(View.GetFeatureLevel(), MAX_STATIC_TEXCOORDS, LightMapCoordinateIndex);
		DynamicMeshBuilder.AddVertices(Verts);
		DynamicMeshBuilder.AddTriangles(Indices);

		FMeshBatch& MeshElement = *RenderContext.Alloc<FMeshBatch>();
		FMeshBuilderOneFrameResources& OneFrameResource = *RenderContext.Alloc<FMeshBuilderOneFrameResources>();

		// Provide custom primitive params to prevent world position offset from moving the mesh
		FPrimitiveUniformShaderParameters PrimitiveParams = FPrimitiveUniformShaderParametersBuilder{}
			.Defaults()
				.LocalToWorld(FMatrix::Identity)
				.ActorWorldPosition(FMatrix::Identity.GetOrigin())
				.WorldBounds(FBoxSphereBounds(EForceInit::ForceInit))
				.LocalBounds(FBoxSphereBounds(EForceInit::ForceInit))
				.ReceivesDecals(false)
				.OutputVelocity(true)
				.EvaluateWorldPositionOffset(false)
			.Build();
		DynamicMeshBuilder.GetMeshElement(PrimitiveParams, Data.MaterialRenderProxy, SDPG_Foreground, true, 0, OneFrameResource, MeshElement);

		check(OneFrameResource.IsValidForRendering());

		Data.LCI->CreatePrecomputedLightingUniformBuffer_RenderingThread(View.GetFeatureLevel());
		MeshElement.LCI = Data.LCI;
		MeshElement.ReverseCulling = false;

#if SHOW_WIREFRAME_MESH
		MeshElement.bWireframe = true;
#endif

		GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, View, MeshElement, false /*bIsHitTesting*/, FHitProxyId());
	}

	virtual bool Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas)
	{
		checkSlow(Data);
		// current render target set for the canvas
		const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
		FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

		// make a temporary view
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = Data->ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = Data->Transform.GetMatrix();
		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.OverlayColor = FLinearColor::White;

		FSceneView* View = new FSceneView(ViewInitOptions);

		RenderMaterial(RenderContext, DrawRenderState, *View, *Data);

		RenderContext.DeferredDelete(View);

		if (Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender)
		{
			RenderContext.DeferredDelete(Data);
			Data = nullptr;
		}
		return true;
	}

	virtual bool Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope)
	{
		checkSlow(Data);
		// current render target set for the canvas
		const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
		FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

		// make a temporary view
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = Data->ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = Data->Transform.GetMatrix();
		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.OverlayColor = FLinearColor::White;

		FSceneView* View = new FSceneView(ViewInitOptions);

		struct FDrawMaterialParameters
		{
			FSceneView* View;
			FRenderData* RenderData;
			uint32 AllowedCanvasModes;
		};
		FDrawMaterialParameters DrawMaterialParameters =
		{
			View,
			Data,
			Canvas->GetAllowedModes()
		};

		FDrawMaterialParameters Parameters = DrawMaterialParameters;
		RenderScope.EnqueueRenderCommand(
			[Parameters](FCanvasRenderContext& RenderContext)
		{
			FMeshPassProcessorRenderState DrawRenderState;

			// disable depth test & writes
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

			RenderMaterial(RenderContext, DrawRenderState, *Parameters.View, *Parameters.RenderData);

			RenderContext.DeferredDelete(Parameters.View);
			if (Parameters.AllowedCanvasModes & FCanvas::Allow_DeleteOnRender)
			{
				RenderContext.DeferredDelete(Parameters.RenderData);
			}
		});
		if (Canvas->GetAllowedModes() & FCanvas::Allow_DeleteOnRender)
		{
			Data = nullptr;
		}
		return true;
	}
};

bool FMeshRenderer::RenderMaterial(struct FMaterialMergeData& InMaterialData, FMaterialRenderProxy* InMaterialProxy, EMaterialProperty InMaterialProperty, UTextureRenderTarget2D* InRenderTarget, TArray<FColor>& OutBMP)
{
	check(IsInGameThread());
	check(InRenderTarget);
	FTextureRenderTargetResource* RTResource = InRenderTarget->GameThread_GetRenderTargetResource();

	{
		// Create a canvas for the render target and clear it to black
		FCanvas Canvas(RTResource, NULL, FGameTime::GetTimeSinceAppStart(), GMaxRHIFeatureLevel);

#if 0	// original FFlattenMaterial code - kept here for comparison

#if !SHOW_WIREFRAME_MESH
		Canvas.Clear(InRenderTarget->ClearColor);
#else
		Canvas.Clear(FLinearColor::Yellow);
#endif

		FVector2D UV0(InMaterialData.TexcoordBounds.Min.X, InMaterialData.TexcoordBounds.Min.Y);
		FVector2D UV1(InMaterialData.TexcoordBounds.Max.X, InMaterialData.TexcoordBounds.Max.Y);
		FCanvasTileItem TileItem(FVector2D(0.0f, 0.0f), InMaterialProxy, FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY), UV0, UV1);
		TileItem.bFreezeTime = true;
		Canvas.DrawItem(TileItem);

		Canvas.Flush_GameThread();
#else

		// create ViewFamily
		const FRenderTarget* CanvasRenderTarget = Canvas.GetRenderTarget();
		FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(
			CanvasRenderTarget,
			NULL,
			FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime()));
		
#if !SHOW_WIREFRAME_MESH
		Canvas.Clear(InRenderTarget->ClearColor);
#else
		Canvas.Clear(FLinearColor::Yellow);
#endif

		// add item for rendering
		FMeshMaterialRenderItem2::EnqueueMaterialRender(
			&Canvas,
			&ViewFamily,
			InMaterialData.Mesh,
			InMaterialData.LODData,
			InMaterialData.LightMapIndex,
			InMaterialData.MaterialIndex,
			InMaterialData.TexcoordBounds,
			InMaterialData.TexCoords,
			FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY),
			InMaterialProxy,
			InMaterialData.LightMap,
			InMaterialData.ShadowMap,
			InMaterialData.Buffer
			);

		// In case of running commandlet the RHI is not fully set up on first flush so do it twice TODO 
		static bool TempForce = true;
		if (IsRunningCommandlet() && TempForce)
		{
			Canvas.Flush_GameThread();
			TempForce = false;
		}

		// rendering is performed here
		Canvas.Flush_GameThread();
#endif

		FlushRenderingCommands();
		Canvas.SetRenderTarget_GameThread(NULL);
		FlushRenderingCommands();
	}

	bool bNormalmap = (InMaterialProperty == MP_Normal);
	FReadSurfaceDataFlags ReadPixelFlags(bNormalmap ? RCM_SNorm : RCM_UNorm);
	ReadPixelFlags.SetLinearToGamma(false);

	bool result = false;

	if (InMaterialProperty != MP_EmissiveColor)
	{
		// Read normal color image
		result = RTResource->ReadPixels(OutBMP, ReadPixelFlags);
	}
	else
	{
		// Read HDR emissive image
		TArray<FFloat16Color> Color16;
		result = RTResource->ReadFloat16Pixels(Color16);
		// Find color scale value
		float MaxValue = 0;
		for (int32 PixelIndex = 0; PixelIndex < Color16.Num(); PixelIndex++)
		{
			FFloat16Color& Pixel16 = Color16[PixelIndex];
			float R = Pixel16.R.GetFloat();
			float G = Pixel16.G.GetFloat();
			float B = Pixel16.B.GetFloat();
			float Max = FMath::Max3(R, G, B);
			if (Max > MaxValue)
			{
				MaxValue = Max;
			}
		}
		if (MaxValue <= 0.01f)
		{
			// Black emissive, drop it
			return false;
		}
		// Now convert Float16 to Color
		OutBMP.SetNumUninitialized(Color16.Num());
		float Scale = 255.0f / MaxValue;
		for (int32 PixelIndex = 0; PixelIndex < Color16.Num(); PixelIndex++)
		{
			FFloat16Color& Pixel16 = Color16[PixelIndex];
			FColor& Pixel8 = OutBMP[PixelIndex];
			Pixel8.R = (uint8)FMath::RoundToInt(Pixel16.R.GetFloat() * Scale);
			Pixel8.G = (uint8)FMath::RoundToInt(Pixel16.G.GetFloat() * Scale);
			Pixel8.B = (uint8)FMath::RoundToInt(Pixel16.B.GetFloat() * Scale);
		}
		
	}

	FMaterialBakingHelpers::PerformUVBorderSmear(OutBMP, InRenderTarget->GetSurfaceWidth(), InRenderTarget->GetSurfaceHeight(), bNormalmap);
#ifdef SAVE_INTERMEDIATE_TEXTURES
	FString FilenameString = FString::Printf(
		TEXT( "D:/TextureTest/%s-mat%d-prop%d.bmp"),
		*InMaterialProxy->GetFriendlyName(), InMaterialData.MaterialIndex, (int32)InMaterialProperty);
	FFileHelper::CreateBitmap(*FilenameString, InRenderTarget->GetSurfaceWidth(), InRenderTarget->GetSurfaceHeight(), OutBMP.GetData());
#endif // SAVE_INTERMEDIATE_TEXTURES
	return result;
}

bool FMeshRenderer::RenderMaterialTexCoordScales(struct FMaterialMergeData& InMaterialData, FMaterialRenderProxy* InMaterialProxy, UTextureRenderTarget2D* InRenderTarget, TArray<FFloat16Color>& OutScales)
{
	check(IsInGameThread());
	check(InRenderTarget);

	// Create a canvas for the render target and clear it to transparent
	FTextureRenderTargetResource* RTResource = InRenderTarget->GameThread_GetRenderTargetResource();
	FCanvas Canvas(RTResource, NULL, FGameTime::GetTimeSinceAppStart(), GMaxRHIFeatureLevel);
	const FRenderTarget* CanvasRenderTarget = Canvas.GetRenderTarget();
	Canvas.Clear(FLinearColor::Transparent);

	// Set show flag view mode to output tex coord scale
	FEngineShowFlags ShowFlags(ESFIM_Game);
	ApplyViewMode(VMI_MaterialTextureScaleAccuracy, false, ShowFlags);
	ShowFlags.OutputMaterialTextureScales = true; // This will bind the DVSM_OutputMaterialTextureScales

	FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(CanvasRenderTarget, nullptr, ShowFlags)
		.SetTime(FGameTime()));

	// The next line ensures a constant view vector of (0,0,1) for all pixels. Required because here SVPositionToTranslatedWorld is identity, making excessive view angle increase per pixel.
	// That creates bad side effects for anything that depends on the view vector, like parallax or bump offset mappings. For those, we want the tangent
	// space view vector to be perpendicular to the surface in order to generate the same results as if the feature was turned off. Which gives the good results
	// since any sub height sampling would in pratice requires less and less texture resolution, where as we are only concerned about the highest resolution the material needs.
	// This can be seen in the debug view mode, by a checkboard of white and cyan (up to green) values. The white value meaning the highest resolution taken is the good one
	// (blue meaning the texture has more resolution than required). Checkboard are only possible when a texture is sampled several times, like in parallax.
	//
	// Additionnal to affecting the view vector, it also forces a constant world position value, zeroing any textcoord scales that depends on the world position (as the UV don't change).
	// This is alright thought since the uniform quad can obviously not compute a valid mapping for world space texture mapping (only rendering the mesh at its world position could fix that).
	// The zero scale will be caught as an error, and the computed scale will fallback to 1.f
	ViewFamily.bNullifyWorldSpacePosition = true;

	// add item for rendering
	FMeshMaterialRenderItem2::EnqueueMaterialRender(
		&Canvas,
		&ViewFamily,
		InMaterialData.Mesh,
		InMaterialData.LODData,
		InMaterialData.LightMapIndex,
		InMaterialData.MaterialIndex,
		InMaterialData.TexcoordBounds,
		InMaterialData.TexCoords,
		FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY),
		InMaterialProxy,
		InMaterialData.LightMap,
		InMaterialData.ShadowMap,
		InMaterialData.Buffer
		);

	// rendering is performed here
	Canvas.Flush_GameThread();

	FlushRenderingCommands();
	Canvas.SetRenderTarget_GameThread(NULL);
	FlushRenderingCommands();

	return RTResource->ReadFloat16Pixels(OutScales);
}
