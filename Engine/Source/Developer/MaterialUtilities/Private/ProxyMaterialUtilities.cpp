// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Engine/MaterialMerging.h"
#include "StaticParameterSet.h"
#include "Materials/Material.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "MaterialUtilities.h"
#include "Materials/MaterialInstanceConstant.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "ProxyMaterialUtilities"

DEFINE_LOG_CATEGORY_STATIC(LogProxyMaterialUtilities, Log, All);

// Find the parameter name to use with the provided material, for the given property
static TArray<FString> GetPotentialParamNames(EFlattenMaterialProperties InProperty)
{
	switch(InProperty)
	{
		case EFlattenMaterialProperties::Diffuse: return { "BaseColor", "Diffuse" };
		case EFlattenMaterialProperties::Normal: return { "Normal" };
		case EFlattenMaterialProperties::Metallic: return { "Metallic" };
		case EFlattenMaterialProperties::Roughness: return { "Roughness" };
		case EFlattenMaterialProperties::Specular: return { "Specular" };
		case EFlattenMaterialProperties::Opacity: return { "Opacity" };
		case EFlattenMaterialProperties::OpacityMask: return { "OpacityMask" };
		case EFlattenMaterialProperties::AmbientOcclusion: return { "AmbientOcclusion" };
		case EFlattenMaterialProperties::Emissive: return { "EmissiveColor", "Emissive" };
	}

	return TArray<FString>();
}

static EMaterialParameterType GetConstantParamType(EFlattenMaterialProperties InProperty)
{
	EMaterialParameterType ParamType = EMaterialParameterType::None;

	switch (InProperty)
	{
	case EFlattenMaterialProperties::Metallic:
	case EFlattenMaterialProperties::Roughness:
	case EFlattenMaterialProperties::Specular:
	case EFlattenMaterialProperties::Opacity:
	case EFlattenMaterialProperties::OpacityMask:
	case EFlattenMaterialProperties::AmbientOcclusion:
		ParamType = EMaterialParameterType::Scalar;
		break;

	case EFlattenMaterialProperties::Diffuse:
	case EFlattenMaterialProperties::Emissive:
		ParamType = EMaterialParameterType::Vector;
		break;
	}

	return ParamType;
}	

// Find the parameter name to use with the provided material, for the given property
static bool GetMatchingParamName(EFlattenMaterialProperties InProperty, const UMaterialInterface* InBaseMaterial, FString& OutParamName, TArray<FString>* OutMissingNames = nullptr)
{
	const TArray<FString> PotentialNames = GetPotentialParamNames(InProperty);

	// Missing names, for error reporting
	FString MissingNames;

	for (const FString& PotentialName : PotentialNames)
	{
		const FName TextureName(PotentialName + TEXT("Texture"));
		const FName ConstName(PotentialName + TEXT("Const"));
		const FName UseTexture(TEXT("Use") + PotentialName);

		UTexture* DefaultTexture = nullptr;
		bool DefaultSwitchValue = false;
		float DefaultScalarValue;
		FLinearColor DefaultVectorValue;
		FGuid ExpressionGuid;

		if (!MissingNames.IsEmpty())
		{
			MissingNames += TEXT("|");
		}

		bool bHasRequiredParams = InBaseMaterial->GetTextureParameterValue(TextureName, DefaultTexture);

		switch(GetConstantParamType(InProperty))
		{
		case EMaterialParameterType::Scalar:
			MissingNames += UseTexture.ToString() + "+" + TextureName.ToString() + "+" + ConstName.ToString();
			bHasRequiredParams &= InBaseMaterial->GetStaticSwitchParameterDefaultValue(UseTexture, DefaultSwitchValue, ExpressionGuid) && 
									InBaseMaterial->GetScalarParameterDefaultValue(ConstName, DefaultScalarValue);
			break;

		case EMaterialParameterType::Vector:
			MissingNames += UseTexture.ToString() + "+" + TextureName.ToString() + "+" + ConstName.ToString();
			bHasRequiredParams &= InBaseMaterial->GetStaticSwitchParameterDefaultValue(UseTexture, DefaultSwitchValue, ExpressionGuid) &&
									InBaseMaterial->GetVectorParameterDefaultValue(ConstName, DefaultVectorValue);
			break;

		case EMaterialParameterType::None:
			MissingNames += UseTexture.ToString();
			break;
		}

		if (bHasRequiredParams)
		{
			OutParamName = PotentialName;
			return true;
		}
	}

	if (OutMissingNames != nullptr)
	{
		OutMissingNames->Add(MissingNames);
	}

	OutParamName = "";
	return false;
}

static FString GetMatchingParamName(EFlattenMaterialProperties InProperty, UMaterialInterface* InBaseMaterial)
{
	FString ParamName = "";
	if (!GetMatchingParamName(InProperty, InBaseMaterial, ParamName))
	{
		UE_LOGF(LogProxyMaterialUtilities, Fatal, "Invalid base material, should have been rejected by IsValidFlattenMaterial()");
	}
	return ParamName;
}

FString FMaterialUtilities::GetFlattenMaterialTextureName(EFlattenMaterialProperties InProperty, UMaterialInterface* InBaseMaterial)
{
	return GetMatchingParamName(InProperty, InBaseMaterial) + TEXT("Texture");
}

// Validate that the provided material has all the required parameters needed to be considered a flattening material
// If not, report what is missing
bool FMaterialUtilities::IsValidFlattenMaterial(const UMaterialInterface* InBaseMaterial)
{
	if (InBaseMaterial != nullptr)
	{
		TArray<FGuid> ParameterIds;
		TArray<FString> MissingParameters;


		FString ParamName;
		GetMatchingParamName(EFlattenMaterialProperties::Diffuse, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::Normal, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::Metallic, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::Roughness, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::Specular, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::Opacity, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::OpacityMask, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::AmbientOcclusion, InBaseMaterial, ParamName, &MissingParameters);
		GetMatchingParamName(EFlattenMaterialProperties::Emissive, InBaseMaterial, ParamName, &MissingParameters);

		auto NameCheckLambda = [&MissingParameters](const TArray<FMaterialParameterInfo>& InCheck, const TArray<FName>& InRequired)
		{
			for (const FName& Name : InRequired)
			{
				if (!InCheck.ContainsByPredicate([Name](const FMaterialParameterInfo& ParamInfo) { return (ParamInfo.Name == Name); }))
				{
					MissingParameters.Add(Name.ToString());
				}
			}
		};

		TArray<FMaterialParameterInfo> TextureParameterInfos;
		TArray<FName> RequiredTextureNames = { TEXT("PackedTexture") };
		InBaseMaterial->GetAllTextureParameterInfo(TextureParameterInfos, ParameterIds);
		NameCheckLambda(TextureParameterInfos, RequiredTextureNames);

		TArray<FMaterialParameterInfo> ScalarParameterInfos;
		TArray<FName> RequiredScalarNames = { TEXT("EmissiveScale") };
		InBaseMaterial->GetAllScalarParameterInfo(ScalarParameterInfos, ParameterIds);
		NameCheckLambda(ScalarParameterInfos, RequiredScalarNames);

		TArray<FMaterialParameterInfo> StaticSwitchParameterInfos;
		TArray<FName> RequiredSwitchNames = { TEXT("PackMetallic"), TEXT("PackSpecular"), TEXT("PackRoughness") };
		InBaseMaterial->GetAllStaticSwitchParameterInfo(StaticSwitchParameterInfos, ParameterIds);
		NameCheckLambda(StaticSwitchParameterInfos, RequiredSwitchNames);

		if (MissingParameters.Num() > 0)
		{
			FString MissingNamesString;
			for (const FString& Name : MissingParameters)
			{
				if (!MissingNamesString.IsEmpty())
				{
					MissingNamesString += ", ";
					MissingNamesString += Name;
				}
				else
				{
					MissingNamesString += Name;
				}
			}

			UE_LOGF(LogProxyMaterialUtilities, Warning, "Material %ls is missing required Material Parameters %ls", *InBaseMaterial->GetName(), *MissingNamesString);

			return false;
		}
		else
		{
			return true;
		}
	}

	return false;
}

static const bool CalculatePackedTextureData(const FFlattenMaterial& InMaterial, bool& bOutPackMetallic, bool& bOutPackSpecular, bool& bOutPackRoughness, int32& OutNumSamples, FIntPoint& OutSize)
{
	// Whether or not a material property is baked down
	const bool bHasMetallic = InMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Metallic) && !InMaterial.IsPropertyConstant(EFlattenMaterialProperties::Metallic);
	const bool bHasRoughness = InMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Roughness) && !InMaterial.IsPropertyConstant(EFlattenMaterialProperties::Roughness);
	const bool bHasSpecular = InMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Specular) && !InMaterial.IsPropertyConstant(EFlattenMaterialProperties::Specular);

	// Check for same texture sizes
	bool bSameTextureSize = false;

	// Determine whether or not the properties sizes match


	const FIntPoint MetallicSize = InMaterial.GetPropertySize(EFlattenMaterialProperties::Metallic);
	const FIntPoint SpecularSize = InMaterial.GetPropertySize(EFlattenMaterialProperties::Specular);
	const FIntPoint RoughnessSize = InMaterial.GetPropertySize(EFlattenMaterialProperties::Roughness);

	bSameTextureSize = (MetallicSize == RoughnessSize) || (MetallicSize == SpecularSize);
	if (bSameTextureSize)
	{
		OutSize = MetallicSize;
		OutNumSamples = InMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Num();
	}
	else
	{
		bSameTextureSize = (RoughnessSize == SpecularSize);
		if (bSameTextureSize)
		{
			OutSize = RoughnessSize;
			OutNumSamples = InMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Num();
		}
	}

	// Now that we know if the data matches determine whether or not we should pack the properties
	int32 NumPacked = 0;
	if (OutNumSamples != 0)
	{
		bOutPackMetallic = bHasMetallic ? (OutNumSamples == InMaterial.GetPropertySamples(EFlattenMaterialProperties::Metallic).Num()) : false;
		NumPacked += (bOutPackMetallic) ? 1 : 0;
		bOutPackRoughness = bHasRoughness ? (OutNumSamples == InMaterial.GetPropertySamples(EFlattenMaterialProperties::Roughness).Num()) : false;
		NumPacked += (bOutPackRoughness) ? 1 : 0;
		bOutPackSpecular = bHasSpecular ? (OutNumSamples == InMaterial.GetPropertySamples(EFlattenMaterialProperties::Specular).Num()) : false;
		NumPacked += (bOutPackSpecular) ? 1 : 0;
	}
	else
	{
		bOutPackMetallic = bOutPackRoughness = bOutPackSpecular = false;
	}

	// Need atleast two properties to pack
	return NumPacked >= 2;
}

UMaterialInstanceConstant* FMaterialUtilities::CreateFlattenMaterialInstance(UPackage* InOuter, const FMaterialProxySettings& InMaterialProxySettings, UMaterialInterface* InBaseMaterial, const FFlattenMaterial& FlattenMaterial, const FString& AssetBasePath, const FString& AssetBaseName, TArray<UObject*>& OutAssetsToSync, FMaterialUpdateContext* MaterialUpdateContext)
{
	UMaterialInterface* BaseMaterial = InBaseMaterial;
	
	if (!IsValidFlattenMaterial(InBaseMaterial))
	{
		BaseMaterial = GEngine->DefaultFlattenMaterial;
	} 

	UMaterialInstanceConstant* OutMaterial = FMaterialUtilities::CreateInstancedMaterial(BaseMaterial, InOuter, AssetBasePath + AssetBaseName, RF_Public | RF_Standalone);
	OutAssetsToSync.Add(OutMaterial);

	OutMaterial->BasePropertyOverrides.TwoSided = FlattenMaterial.bTwoSided && InMaterialProxySettings.bAllowTwoSidedMaterial;
	OutMaterial->BasePropertyOverrides.bOverride_TwoSided = (FlattenMaterial.bTwoSided != false) && InMaterialProxySettings.bAllowTwoSidedMaterial;
	OutMaterial->BasePropertyOverrides.bIsThinSurface = (FlattenMaterial.bIsThinSurface != false);
	OutMaterial->BasePropertyOverrides.bOverride_bIsThinSurface = (FlattenMaterial.bIsThinSurface != false);
	OutMaterial->BasePropertyOverrides.DitheredLODTransition = FlattenMaterial.bDitheredLODTransition;
	OutMaterial->BasePropertyOverrides.bOverride_DitheredLODTransition = FlattenMaterial.bDitheredLODTransition != false;

	if (InMaterialProxySettings.BlendMode != BLEND_Opaque)
	{
		OutMaterial->BasePropertyOverrides.bOverride_BlendMode = true;
		OutMaterial->BasePropertyOverrides.BlendMode = InMaterialProxySettings.BlendMode;
	}

	bool bPackMetallic, bPackSpecular, bPackRoughness;
	int32 NumSamples = 0;
	FIntPoint PackedSize;
	const bool bPackTextures = CalculatePackedTextureData(FlattenMaterial, bPackMetallic, bPackSpecular, bPackRoughness, NumSamples, PackedSize);

	FStaticParameterSet NewStaticParameterSet;

	if(FlattenMaterial.UVChannel != 0)
	{
		// If the used texture coordinate was not the default UV0 set the appropriate one on the instance material
		FStaticSwitchParameter SwitchParameter;
		SwitchParameter.ParameterInfo.Name = TEXT("UseCustomUV");
		SwitchParameter.Value = true;
		SwitchParameter.bOverride = true;
		NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);

		SwitchParameter.ParameterInfo.Name = *(TEXT("UseUV") + FString::FromInt(FlattenMaterial.UVChannel));
		NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
	}

	auto CreateTextureFromDefault = [&](const FName TextureName, const FString& AssetLongName, FIntPoint Size, const TArray<FColor>& Samples)
	{
		FCreateTexture2DParameters CreateParams;
		CreateParams.bSRGB = false;
		CreateParams.bVirtualTexture = false;
		CreateParams.CompressionSettings = TextureCompressionSettings::TC_Default;
		CreateParams.TextureGroup = TextureGroup::TEXTUREGROUP_World;

		UTexture* DefaultTexture = nullptr;
		OutMaterial->GetTextureParameterValue(TextureName, DefaultTexture);
		if (ensure(DefaultTexture))
		{
			CreateParams.bSRGB = DefaultTexture->SRGB;
			CreateParams.bVirtualTexture = DefaultTexture->VirtualTextureStreaming;
			CreateParams.CompressionSettings = DefaultTexture->CompressionSettings;
			CreateParams.TextureGroup = DefaultTexture->LODGroup;
		}

		return FMaterialUtilities::CreateTexture(InOuter, AssetLongName, Size, Samples, CreateParams, RF_Public | RF_Standalone);
	};

	auto SetTextureParam = [&](EFlattenMaterialProperties FlattenProperty)
	{
		if (FlattenMaterial.DoesPropertyContainData(FlattenProperty) && !FlattenMaterial.IsPropertyConstant(FlattenProperty))
		{
			FString PropertyName = GetMatchingParamName(FlattenProperty, OutMaterial);

			const FName TextureName(PropertyName + TEXT("Texture"));
			const FName UseTexture(TEXT("Use") + PropertyName);

			UTexture2D* Texture = CreateTextureFromDefault(TextureName, AssetBasePath + TEXT("T_") + AssetBaseName + TEXT("_") + PropertyName, FlattenMaterial.GetPropertySize(FlattenProperty), FlattenMaterial.GetPropertySamples(FlattenProperty));

			FStaticSwitchParameter SwitchParameter;
			SwitchParameter.ParameterInfo.Name = UseTexture;
			SwitchParameter.Value = true;
			SwitchParameter.bOverride = true;
			NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);

			OutMaterial->SetTextureParameterValueEditorOnly(TextureName, Texture);
			
			OutAssetsToSync.Add(Texture);
		}
	};

	auto SetTextureParamConstVector = [&](EFlattenMaterialProperties FlattenProperty)
	{
		if (FlattenMaterial.DoesPropertyContainData(FlattenProperty) && !FlattenMaterial.IsPropertyConstant(FlattenProperty))
		{
			SetTextureParam(FlattenProperty);
		} 
		else
		{
			FString PropertyName = GetMatchingParamName(FlattenProperty, OutMaterial);

			const FName ConstName(PropertyName + TEXT("Const"));
			OutMaterial->SetVectorParameterValueEditorOnly(ConstName, FlattenMaterial.GetPropertySamples(FlattenProperty)[0]);
		} 
	};

	auto SetTextureParamConstScalar = [&](EFlattenMaterialProperties FlattenProperty, float ConstantValue)
	{
		if (FlattenMaterial.DoesPropertyContainData(FlattenProperty) && !FlattenMaterial.IsPropertyConstant(FlattenProperty))
		{
			SetTextureParam(FlattenProperty);
		}
		else
		{
			FString PropertyName = GetMatchingParamName(FlattenProperty, OutMaterial);

			const FName ConstName(PropertyName + TEXT("Const"));
			FLinearColor Colour = FlattenMaterial.IsPropertyConstant(FlattenProperty) ? FLinearColor::FromSRGBColor(FlattenMaterial.GetPropertySamples(FlattenProperty)[0]) : FLinearColor(ConstantValue, 0, 0, 0);
			OutMaterial->SetScalarParameterValueEditorOnly(ConstName, Colour.R);
		}
	};

	// Load textures and set switches accordingly
	if (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::Diffuse) && !(FlattenMaterial.IsPropertyConstant(EFlattenMaterialProperties::Diffuse) && FlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Diffuse)[0] == FColor::Black))
	{
		SetTextureParamConstVector(EFlattenMaterialProperties::Diffuse);
	}

	const FIntPoint NormalTextureSize = FlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Normal);
	if (NormalTextureSize.X * NormalTextureSize.Y > 1)
	{
		SetTextureParam(EFlattenMaterialProperties::Normal);
	}

	// Determine whether or not specific material properties are packed together into one texture (requires at least two to match (number of samples and texture size) in order to be packed
	if (!bPackMetallic && (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::Metallic) || !InMaterialProxySettings.bMetallicMap))
	{
		SetTextureParamConstScalar(EFlattenMaterialProperties::Metallic, InMaterialProxySettings.MetallicConstant);
	}

	if (!bPackRoughness && (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::Roughness) || !InMaterialProxySettings.bRoughnessMap))
	{
		SetTextureParamConstScalar(EFlattenMaterialProperties::Roughness, InMaterialProxySettings.RoughnessConstant);
	}

	if (!bPackSpecular && (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::Specular) || !InMaterialProxySettings.bSpecularMap))
	{
		SetTextureParamConstScalar(EFlattenMaterialProperties::Specular, InMaterialProxySettings.SpecularConstant);
	}

	if (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::Opacity) || !InMaterialProxySettings.bOpacityMap)
	{
		SetTextureParamConstScalar(EFlattenMaterialProperties::Opacity, InMaterialProxySettings.OpacityConstant);
	}

	if (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::OpacityMask) || !InMaterialProxySettings.bOpacityMaskMap)
	{
		SetTextureParamConstScalar(EFlattenMaterialProperties::OpacityMask, InMaterialProxySettings.OpacityMaskConstant);
	}

	if (FlattenMaterial.ShouldGenerateDataForProperty(EFlattenMaterialProperties::AmbientOcclusion) || !InMaterialProxySettings.bAmbientOcclusionMap)
	{
		SetTextureParamConstScalar(EFlattenMaterialProperties::AmbientOcclusion, InMaterialProxySettings.AmbientOcclusionConstant);
	}

	// Handle the packed texture if applicable
	if (bPackTextures)
	{
		TArray<FColor> MergedTexture;
		MergedTexture.AddZeroed(NumSamples);

		// Merge properties into one texture using the separate colour channels
		const EFlattenMaterialProperties Properties[3] = { EFlattenMaterialProperties::Metallic , EFlattenMaterialProperties::Roughness, EFlattenMaterialProperties::Specular };

		//Property that is not part of the pack (because of a different size), will see is reserve pack space fill with Black Color.
		const bool PropertyShouldBePack[3] = { bPackMetallic , bPackRoughness , bPackSpecular };

		// Red mask (all properties are rendered into the red channel)
		FColor NonAlphaRed = FColor::Red;
		NonAlphaRed.A = 0;
		const uint32 ColorMask = NonAlphaRed.DWColor();
		const uint32 Shift[3] = { 0, 8, 16 };
		for (int32 PropertyIndex = 0; PropertyIndex < 3; ++PropertyIndex)
		{
			const EFlattenMaterialProperties Property = Properties[PropertyIndex];
			const bool HasProperty = PropertyShouldBePack[PropertyIndex] && FlattenMaterial.DoesPropertyContainData(Property) && !FlattenMaterial.IsPropertyConstant(Property);

			if (HasProperty)
			{
				const TArray<FColor>& PropertySamples = FlattenMaterial.GetPropertySamples(Property);
				// OR masked values (samples initialized to zero, so no random data)
				for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
				{
					// Black adds the alpha + red channel value shifted into the correct output channel
					MergedTexture[SampleIndex].DWColor() |= (FColor::Black.DWColor() + ((PropertySamples[SampleIndex].DWColor() & ColorMask) >> Shift[PropertyIndex]));
				}
			}
		}

		// Create texture using the merged property data
		const FName PackedTextureName(TEXT("PackedTexture"));
		UTexture2D* PackedTexture = CreateTextureFromDefault(PackedTextureName, AssetBasePath + TEXT("T_") + AssetBaseName + TEXT("_MRS"), PackedSize, MergedTexture);
		checkf(PackedTexture, TEXT("Failed to create texture"));
		OutAssetsToSync.Add(PackedTexture);

		auto PackProperty = [&NewStaticParameterSet](const FString& PropertyName, bool bEnable)
		{
			FStaticSwitchParameter SwitchParameter;
			SwitchParameter.ParameterInfo.Name = FName(FString(TEXT("Pack")) + PropertyName);
			SwitchParameter.Value = bEnable;
			SwitchParameter.bOverride = true;
			NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
			SwitchParameter.ParameterInfo.Name = FName(FString(TEXT("Use")) + PropertyName);
			NewStaticParameterSet.StaticSwitchParameters.Add(SwitchParameter);
		};
		
		// Setup switches for whether or not properties will be packed into one texture
		PackProperty(TEXT("Metallic"), bPackMetallic);
		PackProperty(TEXT("Specular"), bPackSpecular);
		PackProperty(TEXT("Roughness"), bPackRoughness);

		// Set up switch and texture values
		OutMaterial->SetTextureParameterValueEditorOnly(PackedTextureName, PackedTexture);
	}

	// Emissive is a special case due to the scaling variable
	const FIntPoint EmissiveTextureSize = FlattenMaterial.GetPropertySize(EFlattenMaterialProperties::Emissive);
	if (FlattenMaterial.DoesPropertyContainData(EFlattenMaterialProperties::Emissive) && !(EmissiveTextureSize.X * EmissiveTextureSize.Y == 1 && FlattenMaterial.GetPropertySamples(EFlattenMaterialProperties::Emissive)[0] == FColor::Black))
	{
		SetTextureParamConstVector(EFlattenMaterialProperties::Emissive);

		if (FlattenMaterial.EmissiveScale != 1.0f)
		{
			FMaterialParameterInfo ParameterInfo(TEXT("EmissiveScale"));
			OutMaterial->SetScalarParameterValueEditorOnly(ParameterInfo, FlattenMaterial.EmissiveScale);
		}
	}

	// Force initializing the static permutations according to the switches we have set
	OutMaterial->UpdateStaticPermutation(NewStaticParameterSet, MaterialUpdateContext);
	OutMaterial->InitStaticPermutation();

	OutMaterial->PostEditChange();

	return OutMaterial;
}	

#undef LOCTEXT_NAMESPACE // "ProxyMaterialUtilities"