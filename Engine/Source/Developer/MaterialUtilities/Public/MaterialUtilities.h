// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/AnyOf.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "SceneTypes.h"
#include "Modules/ModuleInterface.h"
#include "Engine/TextureStreamingTypes.h"
#include "Engine/Texture.h"

#include "LightMap.h"
#include "ShadowMap.h"
#include "ImageUtils.h"

#include "MaterialUtilities.generated.h"

#define UE_API MATERIALUTILITIES_API

class ALandscapeProxy;
class Error;
class ULandscapeComponent;
class UMaterial;
class UMaterialInstanceConstant;
class UMaterialInterface;
class UTexture2D;
class UTextureRenderTarget2D;
class UMaterialOptions;
struct FMaterialProxySettings;
struct FMeshDescription;
class FMaterialUpdateContext;
class FSkeletalMeshLODRenderData;
struct FBakeOutput;
struct FMeshData;
struct FMaterialData;

/* TODO replace this with rendering property enum when extending the system */
UENUM()
enum class EFlattenMaterialProperties : uint8
{
	Diffuse,
	Metallic,
	Specular,
	Roughness,
	Anisotropy,
	Normal,
	Tangent,
	Opacity,
	Emissive,
	SubSurface,
	OpacityMask,
	AmbientOcclusion = 16,
	NumFlattenMaterialProperties
};

/** Struct to store per section info used to populate data after (multiple) meshes are merged together */
struct FSectionInfo
{
	FSectionInfo() : Material(nullptr), MaterialSlotName(NAME_None), MaterialIndex(INDEX_NONE), StartIndex(INDEX_NONE), EndIndex(INDEX_NONE), bProcessed(false), OverlayMaterial(nullptr) {}

	/** Material used by the section */
	class UMaterialInterface* Material;
	/** Name value for the section */
	FName MaterialSlotName;
	/** List of properties enabled for the section (collision, cast shadow etc) */
	TArray<FName> EnabledProperties;
	/** Original index of Material in the source data */
	int32 MaterialIndex;
	/** Index pointing to the start set of mesh indices that belong to this section */
	int32 StartIndex;
	/** Index pointing to the end set of mesh indices that belong to this section */
	int32 EndIndex;
	/** Used while baking out materials, to check which sections are and aren't being baked out */
	bool bProcessed;
	/** Overlay material used by the section */
	class UMaterialInterface* OverlayMaterial;

	bool operator==(const FSectionInfo& Other) const
	{
		return Material == Other.Material && EnabledProperties == Other.EnabledProperties;
	}
};

/** Structure used for storing intermediate baked down material data/samples*/
struct FFlattenMaterial
{
	FFlattenMaterial()
		: RenderSize(0, 0)
		, bTwoSided(false)
		, bIsThinSurface(false)
		, bDitheredLODTransition(false)
		, BlendMode(BLEND_Opaque)
		, EmissiveScale(1.0f)
		, UVChannel(0)
	{
		for (FIntPoint& Size : PropertySizes)
		{
			Size = FIntPoint(ForceInitToZero);
		}
	}

	bool HasData() const
	{
		return Algo::AnyOf(PropertySamples, [](const TArray<FColor>& Samples) { return !Samples.IsEmpty(); });
	}

	/** Release all the sample data */
	void ReleaseData()
	{
		for (TArray<FColor>& Samples : PropertySamples)
		{
			Samples.Empty();
		}
	}
	
	/** Set all alpha channel values with InAlphaValue */
	void FillAlphaValues(const uint8 InAlphaValue)
	{
		for (TArray<FColor>& SampleArray : PropertySamples)
		{
			for (FColor& Sample : SampleArray)
			{
				Sample.A = InAlphaValue;
			}
		}
	}

	const bool DoesPropertyContainData(const EFlattenMaterialProperties Property) const { return GetSamplesEntry(Property).Num() > 0; }

	const bool IsPropertyConstant(const EFlattenMaterialProperties Property) const { return GetSamplesEntry(Property).Num() == 1; }

	const bool ShouldGenerateDataForProperty(const EFlattenMaterialProperties Property) const { return GetSizesEntry(Property).GetMin() > 0; }

	const FIntPoint GetPropertySize(const EFlattenMaterialProperties Property) const{ return GetSizesEntry(Property); }
	void SetPropertySize(const EFlattenMaterialProperties Property, const FIntPoint& InSize) { GetSizesEntry(Property) = InSize; }

	TArray<FColor>& GetPropertySamples(const EFlattenMaterialProperties Property) { return GetSamplesEntry(Property); }
	const TArray<FColor>& GetPropertySamples(const EFlattenMaterialProperties Property) const { return GetSamplesEntry(Property); }

	/** Material Guid */
	FGuid			MaterialId;	
	FIntPoint		RenderSize;

	/** Flag whether or not the material will have to be two-sided */
	bool			bTwoSided;
	/** Flag whether or not the material will have to be thin  */
	bool			bIsThinSurface;
	/** Flag whether or not the material will use dithered LOD transitions */
	bool			bDitheredLODTransition;
	/** Blend mode for the new material */
	EBlendMode		BlendMode;
	/** Scale (maximum baked down value) for the emissive property */
	float			EmissiveScale;
	/** UV channel to use */
	int32			UVChannel;

private:
	FIntPoint& GetSizesEntry(const EFlattenMaterialProperties Property)
	{
		const uint32 Index = (uint32)Property;
		check(Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties);

		static FIntPoint TempSize = FIntPoint::ZeroValue;
		return Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties ? PropertySizes[Index] : TempSize;
	}

	TArray<FColor>& GetSamplesEntry(const EFlattenMaterialProperties Property)
	{
		const uint32 Index = (uint32)Property;
		check(Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties);

		static TArray<FColor> TempArray;
		return Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties ? PropertySamples[Index] : TempArray;
	}

	const FIntPoint& GetSizesEntry(const EFlattenMaterialProperties Property) const
	{
		const uint32 Index = (uint32)Property;
		check(Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties);

		static const FIntPoint TempSize = FIntPoint::ZeroValue;
		return Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties ? PropertySizes[Index] : TempSize;
	}

	const TArray<FColor>& GetSamplesEntry(const EFlattenMaterialProperties Property) const
	{
		const uint32 Index = (uint32)Property;
		check(Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties);

		static const TArray<FColor> TempArray;
		return Index < (uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties ? PropertySamples[Index] : TempArray;
	}

	/** Texture sizes for each individual property*/
	FIntPoint PropertySizes[(uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties];
	/** Baked down texture samples for each individual property*/
	TArray<FColor> PropertySamples[(uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties];
};

/** Export material proxy cache*/
struct FExportMaterialProxyCache
{
	// Material proxies for each property. Note: we're not handling all properties here,
	// so hold only up to MP_Normal inclusive.
	class FMaterialRenderProxy* Proxies[MP_MAX];

	UE_API FExportMaterialProxyCache();
	UE_API ~FExportMaterialProxyCache();

	/** Releasing the cached render proxies */
	UE_API void Release();
};

/** Intermediate material merging data */
struct FMaterialMergeData
{
	/** Material proxy cache, eliminates shader compilations when a material is baked out multiple times for different meshes */
	FExportMaterialProxyCache* ProxyCache;

	/** Input data */
	/** Material that is being baked out */
	class UMaterialInterface* Material;
	/** Raw mesh data used to bake out the material with, optional */
	const struct FMeshDescription* Mesh;
	/** LODModel data used to bake out the material with, optional */
	const FSkeletalMeshLODRenderData* LODData;
	/** Material index to use when the material is baked out using mesh data (face material indices) */
	int32 MaterialIndex;
	/** Optional tex coordinate bounds of original texture coordinates set */
	FBox2D TexcoordBounds;
	/** Optional new set of non-overlapping texture coordinates */
	const TArray<FVector2D>& TexCoords;

	FLightMapRef LightMap;
	FShadowMapRef ShadowMap;
	FUniformBufferRHIRef Buffer;
	int32 LightMapIndex;

	/** Output emissive scale, maximum baked out emissive value (used to scale other samples, 1/EmissiveScale * Sample) */
	float EmissiveScale;

	FMaterialMergeData(
		UMaterialInterface* InMaterial,
		const FMeshDescription* InMesh,
		const FSkeletalMeshLODRenderData* InLODData,
		int32 InMaterialIndex,
		FBox2D InTexcoordBounds,
		const TArray<FVector2D>& InTexCoords)
		: ProxyCache(nullptr)
		, Material(InMaterial)
		, Mesh(InMesh)
		, LODData(InLODData)
		, MaterialIndex(InMaterialIndex)
		, TexcoordBounds(InTexcoordBounds)
		, TexCoords(InTexCoords)
		, LightMapIndex(1)
		, EmissiveScale(0.0f)
	{
		ProxyCache = new FExportMaterialProxyCache();
	}

	~FMaterialMergeData()
	{
		if (ProxyCache)
		{
			delete ProxyCache;
		}
	}
};


class UMaterialInterface;
class UMaterial;
class UTexture2D;
class UTextureRenderTarget2D;
class UWorld;
class ALandscapeProxy;
class ULandscapeComponent;
class FPrimitiveComponentId;
class UMaterialInstanceConstant;
struct FMaterialMergeData;
struct FMeshDescription;

namespace UE::Geometry { class FDynamicMesh3; }

/**
 * Material utilities
 */
class FMaterialUtilities : public IModuleInterface
{
public:
	/** Begin IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	/** End IModuleInterface implementation */

	/**
	* Iterate through all textures used by the material and return the maximum texture resolution used
	*
	* @param MaterialInterface	The material to scan for texture size
	* @param MinimumSize		The minimum size returned
	*
	* @return Size (width and height)
	*/
	static UE_API FIntPoint FindMaxTextureSize(UMaterialInterface* InMaterialInterface, FIntPoint MinimumSize = FIntPoint(1, 1));

	/**
	 * Whether material utilities support exporting specified material blend mode and property 
	 */
	static UE_API bool SupportsExport(bool bIsOpaque, EMaterialProperty InMaterialProperty);

	/**
	 * Flattens specified landscape material
	 *
	 * @param InLandscape			Target landscape
	 * @param OutFlattenMaterial	Output flattened material
	 * @return						Whether operation was successful
	 */
	static UE_API bool ExportLandscapeMaterial(const ALandscapeProxy* InLandscape, FFlattenMaterial& OutFlattenMaterial);

	/**
	 * Flattens specified landscape material
	 *
	 * @param InLandscape			Target landscape
	 * @param HiddenPrimitives		Primitives to hide while rendering scene to texture
	 * @param OutFlattenMaterial	Output flattened material
	 * @return						Whether operation was successful
	 */
	static UE_API bool ExportLandscapeMaterial(const ALandscapeProxy* InLandscape, const TSet<FPrimitiveComponentId>& HiddenPrimitives, FFlattenMaterial& OutFlattenMaterial);
	
	/**
 	 * Generates a texture from an array of samples 
	 *
	 * @param Outer					Outer for the material and texture objects, if NULL new packages will be created for each asset
	 * @param AssetLongName			Long asset path for the new texture
	 * @param Size					Resolution of the texture to generate (must match the number of samples)
	 * @param Samples				Color data for the texture
	 * @param CompressionSettings	Compression settings for the new texture
	 * @param LODGroup				LODGroup for the new texture
	 * @param Flags					ObjectFlags for the new texture
	 * @param bSRGB					Whether to set the bSRGB flag on the new texture
	 * @param SourceGuidHash		(optional) Hash (stored as Guid) to use part of the texture source's DDC key.
	 * @return						The new texture.
	 */
	static UE_API UTexture2D* CreateTexture(UPackage* Outer, const FString& AssetLongName, FIntPoint Size, const TArray<FColor>& Samples, TextureCompressionSettings CompressionSettings, TextureGroup LODGroup, EObjectFlags Flags, bool bSRGB, const FGuid& SourceGuidHash = FGuid());

	/**
	 * Generates a texture from an array of samples
	 *
	 * @param Outer					Outer for the material and texture objects, if NULL new packages will be created for each asset
	 * @param AssetLongName			Long asset path for the new texture
	 * @param Size					Resolution of the texture to generate (must match the number of samples)
	 * @param Samples				Color data for the texture
	 * @param CreateParams			Params about how to set up the texture
	 * @param Flags					ObjectFlags for the new texture
	 * @return						The new texture.
	 */
	static UE_API UTexture2D* CreateTexture(UPackage* Outer, const FString& AssetLongName, FIntPoint Size, const TArray<FColor>& Samples, const FCreateTexture2DParameters& CreateParams, EObjectFlags Flags);

	/**
	 * Creates UMaterial object from a flatten material
	 *
	 * @param Outer					Outer for the material and texture objects, if NULL new packages will be created for each asset
	 * @param BaseName				BaseName for the material and texture objects, should be a long package name in case Outer is not specified
	 * @param Flags					Object flags for the material and texture objects.
	 * @param OutGeneratedAssets	List of generated assets - material, textures
	 * @return						Returns a pointer to the constructed UMaterial object.
	 */
	static UE_API UMaterial* CreateMaterial(const FFlattenMaterial& InFlattenMaterial, UPackage* InOuter, const FString& BaseName, EObjectFlags Flags, const struct FMaterialProxySettings& MaterialProxySettings, TArray<UObject*>& OutGeneratedAssets, const TextureGroup& InTextureGroup = TEXTUREGROUP_World);

	/** 
	* Creates an instanced material based of BaseMaterial
	* @param Outer					Outer for the material and texture objects, if NULL new packages will be created for each asset
	* @param BaseName				BaseName for the material and texture objects, should be a long package name in case Outer is not specified
	* @param Flags					Object flags for the material and texture objects.
	* @return						Returns a pointer to the constructed UMaterialInstanceConstant object.
	*/

	static UE_API UMaterialInstanceConstant* CreateInstancedMaterial(UMaterialInterface* BaseMaterial, UPackage* InOuter, const FString& BaseName, EObjectFlags Flags);

	/**
	* Creates bakes textures for a ULandscapeComponent
	*
	* @param LandscapeComponent		The component to bake textures for
	* @return						Whether operation was successful
	*/
	static UE_API bool ExportBaseColor(ULandscapeComponent* LandscapeComponent, int32 TextureSize, TArray<FColor>& OutSamples);

	/**
	* Creates a FFlattenMaterial instance with the given MaterialProxySettings data
	*
	* @param InMaterialLODSettings  Settings containing how to material should be merged
	* @return						Set up FFlattenMaterial according to the given settings
	*/
	static UE_API FFlattenMaterial CreateFlattenMaterialWithSettings(const FMaterialProxySettings& InMaterialLODSettings);

	/**
	* Analyzes given material to determine how many texture coordinates and whether or not vertex colors are used within the material Graph
	*
	* @param InMaterial 			Material to analyze
	* @param InMaterialSettings 	Settings containing how to material should be merged
	* @param OutNumTexCoords		Number of texture coordinates used across all properties flagged for merging
	* @param OutRequiresVertexData	Flag whether or not Vertex Data is used in the material graph for the properties flagged for merging
	*/
	static UE_API void AnalyzeMaterial(class UMaterialInterface* InMaterial, const struct FMaterialProxySettings& InMaterialSettings, int32& OutNumTexCoords, bool& OutRequiresVertexData);

	static UE_API void AnalyzeMaterial(class UMaterialInterface* InMaterial, const TArray<EMaterialProperty>& Properties, int32& OutNumTexCoords, bool& OutRequiresVertexData);

	/**
	* Remaps material indices where possible to reduce the number of materials required for creating a proxy material
	*
	* @param InMaterials 					List of Material interfaces (non-unique)
	* @param InMeshData						Array of meshes who use the materials in InMaterials
	* @param InMaterialMap					Map of MeshIDAndLOD keys with a material index array as value mapping InMeshData to the InMaterials array
	* @param InMaterialProxySettings		Settings for creating the proxy material
	* @param bBakeVertexData 				Flag whether or not Vertex Data should be baked down
	* @param bMergeMaterials 				Flag whether or not materials with be merged for this mesh
	* @param OutMeshShouldBakeVertexData	Array with Flag for each mesh whether or not Vertex Data should be baked down or is required to
	* @param OutMaterialMap					Map of MeshIDAndLOD keys with a material index array as value mapping InMeshData to the OutMaterials array
	* @param OutMaterials					List of Material interfaces (unique)
	*/
	static UE_API void RemapUniqueMaterialIndices(const TArray<struct FSectionInfo>& InSections, const TArray<struct FRawMeshExt>& InMeshData, const TMap<FIntPoint, TArray<int32> >& InMaterialMap, const FMaterialProxySettings& InMaterialProxySettings, const bool bBakeVertexData, const bool bMergeMaterials, TArray<bool>& OutMeshShouldBakeVertexData, TMap<FIntPoint, TArray<int32> >& OutMaterialMap, TArray<struct FSectionInfo>& OutSections);

	/**
	* Tries to optimize the flatten material's data by picking out constant values for the various properties
	*
	* @param InFlattenMaterial				Flatten material to optimize
	*/
	static UE_API void OptimizeFlattenMaterial(FFlattenMaterial& InFlattenMaterial);

	/**
	* Resizes flatten material's data if applicable by comparing it with the original settings
	*
	* @param InFlattenMaterial				Flatten material to optimize
	*/
	static UE_API void ResizeFlattenMaterial(FFlattenMaterial& InFlattenMaterial, const struct FMeshProxySettings& InMeshProxySettings);

	/** Tries to optimize the sample array (will set to const value if all samples are equal) */
	static UE_API void OptimizeSampleArray(TArray<FColor>& InSamples, FIntPoint& InSampleSize);

	/**
	* Contains errors generated when exporting material texcoord scales. 
	* Used to prevent displaying duplicates, as instances using the same shaders get the same issues.
	*/
	class FExportErrorManager
	{
	public:

		FExportErrorManager(EShaderPlatform InShaderPlatform) : ShaderPlatform(InShaderPlatform) {}

		enum EErrorType
		{
			EET_IncohorentValues,
			EET_NoValues
		};

		/**
		* Register a new error.
		*
		* @param Material			The material having this error.
		* @param TextureName		The texture for which the scale could not be generated.
		* @param RegisterIndex		The register index bound to this texture.
		* @param ErrorType			The issue encountered.
		*/
		UE_API void Register(const UMaterialInterface* Material, FName TextureName, int32 RegisterIndex, EErrorType ErrorType);

		/**
		* Output all errors registered.
		*/
		UE_API void OutputToLog();

	private:

		struct FError
		{
			const FMaterial* Material;
			int32 RegisterIndex;
			EErrorType ErrorType;

			bool operator==(const FError& Rhs) const;
		};

		struct FInstance
		{
			const UMaterialInterface* Material;
			FName TextureName;
		};

		friend uint32 GetTypeHash(const FError& Error);

		EShaderPlatform ShaderPlatform;
		TMap<FError, TArray<FInstance> > ErrorInstances;
	};

	/**
	* Get the material texcoord scales applied on each textures
	*
	* @param InMaterial			Target material
	* @param QualityLevel		Quality level used for the shader profiling.
	* @param FeatureLevel		Feature level used for the shader profiling.
	* @param OutErrors			Manager to log errors (removes duplicates and similar errors)	
	* @return					Whether operation was successful
	*/
	UE_DEPRECATED(5.7, "Please use ExportMaterialUVDensities with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	static UE_API bool ExportMaterialUVDensities(UMaterialInterface* InMaterial, EMaterialQualityLevel::Type QualityLevel, ERHIFeatureLevel::Type FeatureLevel, FExportErrorManager& OutErrors);
	static UE_API bool ExportMaterialUVDensities(UMaterialInterface* InMaterial, EMaterialQualityLevel::Type QualityLevel, EShaderPlatform InShaderPlatform, FExportErrorManager& OutErrors);

	/** Calculates an importance value for the given set of materials according to the texture sampler usage */
	static UE_API void DetermineMaterialImportance(const TArray<UMaterialInterface*>& InMaterials, TArray<float>& OutImportance);
	/** Generates a set of texture boxes according to the given weights and final atlas texture size */
	static UE_API void GeneratedBinnedTextureSquares(const FVector2D DestinationSize, TArray<float>& InTexureWeights, TArray<FBox2D>& OutGeneratedBoxes);

	/** Creates a proxy material and the required texture assets */
	static UE_API UMaterialInterface* CreateProxyMaterialAndTextures(UPackage* OuterPackage, const FString& AssetName, const FBakeOutput& BakeOutput, const FMeshData& MeshData, const FMaterialData& MaterialData, UMaterialOptions* Options);

	/** Creates a proxy material and the required texture assets */
	static UE_API UMaterialInterface* CreateProxyMaterialAndTextures(const FString& PackageName, const FString& AssetName, const FBakeOutput& BakeOutput, const FMeshData& MeshData, const FMaterialData& MaterialData, UMaterialOptions* Options);

	/** Creates a flatten material instance and the required texture assets */
	static UE_API UMaterialInstanceConstant* CreateFlattenMaterialInstance(UPackage* InOuter, const FMaterialProxySettings& InMaterialProxySettings, UMaterialInterface* InBaseMaterial, const FFlattenMaterial& FlattenMaterial, const FString& AssetBasePath, const FString& AssetBaseName, TArray<UObject*>& OutAssetsToSync, FMaterialUpdateContext* MaterialUpdateContext = nullptr);

	/** Compute the required texel density needed to properly represent an object/objects covering the provided world space radius
	* 
	* @param InScreenSize		Screen size at which the objects are expected to be be shown
	* @param InWorldSpaceRadius	World space radius of the objects
	* @return					The required texel density per METER
	*/
	static UE_API float ComputeRequiredTexelDensityFromScreenSize(const float InScreenSize, float InWorldSpaceRadius);

	/** Compute the required texel density needed to properly represent an object/objects covering the provided world space radius
	*
	* @param InDrawDistance		Draw distance at which the objects are expected to be shown
	* @param InWorldSpaceRadius	World space radius of the objects
	* @return					The required texel density per METER
	*/
	static UE_API float ComputeRequiredTexelDensityFromDrawDistance(const float InDrawDistance, float InWorldSpaceRadius);

	/** Compute the required texture size to achieve a target texel density for the given mesh.
	* @param InMesh					The mesh for which we want to create a flatten material.
	* @param InTargetTexelDensity	The target texel density.
	* @return The texture size needed to achieve the required texel density.
	*/
	static UE_API int32 GetTextureSizeFromTargetTexelDensity(const FMeshDescription& InMesh, float InTargetTexelDensity);

	/** Compute the required texture size to achieve a target texel density for the given mesh.
	* @param InMesh					The mesh for which we want to create a flatten material.
	* @param InTargetTexelDensity	The target texel density.
	* @return The texture size needed to achieve the required texel density.
	*/
	static UE_API int32 GetTextureSizeFromTargetTexelDensity(const UE::Geometry::FDynamicMesh3& Mesh, float TargetTexelDensity);

	/** Compute the required texture size to achieve a target texel density, given the world to UV space ratio.
	* @param InMesh3DArea	World space area of the mesh
	* @param InMeshUVArea	UV space area of the mesh
	* @return The texture size needed to achieve the required texel density.
	*/
	static UE_API int32 GetTextureSizeFromTargetTexelDensity(double InMesh3DArea, double InMeshUVArea, double InTargetTexelDensity);

	/** Validate that the provided material has all the required parameters needed to be considered a flattening material */
	static UE_API bool IsValidFlattenMaterial(const UMaterialInterface* InBaseMaterial);

	/** Get the name of the Texture parameter associated with a given flatten material property */
	static UE_API FString GetFlattenMaterialTextureName(EFlattenMaterialProperties InProperty, UMaterialInterface* InBaseMaterial);

private:
	
	/**
	* Creates and add or reuses a RenderTarget from the pool
	*
	* @param bInForceLinearGamma	Whether or not to force linear gamma
	* @param InPixelFormat			Pixel format of the render target
	* @param InTargetSize			Dimensions of the render target
	* @return						Created render target
	*/
	static UE_API UTextureRenderTarget2D* CreateRenderTarget(bool bInForceLinearGamma, bool bNormalMap, EPixelFormat InPixelFormat, FIntPoint& InTargetSize );
	
	/** Clears the pool with available render targets */
	static UE_API void ClearRenderTargetPool();	

	/** Call back for garbage collector, cleans up the RenderTargetPool if CurrentlyRendering is set to false */
	UE_API void OnPreGarbageCollect();
private:
	/** Flag to indicate whether or not a texture is currently being rendered out */
	static UE_API bool CurrentlyRendering;
	/** Pool of available render targets, cached for re-using on consecutive property rendering */
	static UE_API TArray<UTextureRenderTarget2D*> RenderTargetPool;
};

#undef UE_API
