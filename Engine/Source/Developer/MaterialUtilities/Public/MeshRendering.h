// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshRendering.h: Simple mesh rendering implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "SceneTypes.h"

class FMaterialRenderProxy;
class UTextureRenderTarget2D;
struct FMaterialMergeData;

/** Class used as an interface for baking materials to textures using mesh/vertex-data */
class FMeshRenderer
{
public:
	/** Renders out textures for each material property for the given material, using the given mesh data or by using a simple tile rendering approach */
	MATERIALUTILITIES_API static bool RenderMaterial(
		FMaterialMergeData& InMaterialData, 
		FMaterialRenderProxy* InMaterialProxy,
		EMaterialProperty InMaterialProperty, 
		UTextureRenderTarget2D* InRenderTarget, 
		TArray<FColor>& OutBMP
		);	

	/** Renders out texcoord scales */
	MATERIALUTILITIES_API static bool RenderMaterialTexCoordScales(
		FMaterialMergeData& InMaterialData, 
		FMaterialRenderProxy* InMaterialProxy,
		UTextureRenderTarget2D* InRenderTarget, 
		TArray<FFloat16Color>& OutScales
		);	

private:
	/** This class never needs to be instantiated. */
	FMeshRenderer() {}
};
